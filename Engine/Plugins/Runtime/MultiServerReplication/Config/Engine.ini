[/Script/Engine.Engine]
+NetDriverDefinitions=(DefName="MultiServerNetDriver",DriverClassName="/Script/MultiServerReplication.MultiServerNetDriver",DriverClassNameFallback="/Script/MultiServerReplication.MultiServerNetDriver")
+NetDriverDefinitions=(DefName="ProxyNetDriver",DriverClassName="/Script/MultiServerReplication.ProxyNetDriver",DriverClassNameFallback="/Script/MultiServerReplication.ProxyNetDriver")
+NetDriverDefinitions=(DefName="ProxyBackendNetDriver",DriverClassName="/Script/MultiServerReplication.ProxyBackendNetDriver",DriverClassNameFallback="/Script/MultiServerReplication.ProxyBackendNetDriver")
+IrisNetDriverConfigs=(NetDriverDefinition=MultiServerNetDriver, bCanUseIris=true, PreferredReplicationSystem=Iris)
+IrisNetDriverConfigs=(NetDriverDefinition=ProxyNetDriver, bCanUseIris=true)
+IrisNetDriverConfigs=(NetDriverDefinition=ProxyBackendNetDriver, bCanUseIris=true)