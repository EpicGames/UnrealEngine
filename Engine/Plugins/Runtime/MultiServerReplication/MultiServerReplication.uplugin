{
	"FileVersion": 3,
	"FriendlyName": "Multi-server Replication",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Code to help facilitate connecting multiple UE server processes to each other.",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsExperimentalVersion":  true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MultiServerReplication",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MultiServerConfiguration",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		}
	]
}