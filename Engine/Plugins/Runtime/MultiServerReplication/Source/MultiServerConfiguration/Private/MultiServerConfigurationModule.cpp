// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiServerConfigurationModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FMultiServerConfigurationModule, MultiServerConfiguration);