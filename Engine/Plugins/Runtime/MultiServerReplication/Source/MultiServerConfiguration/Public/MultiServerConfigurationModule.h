// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/** See MultiServerNode for more details about multi-server networking as a whole. */
class FMultiServerConfigurationModule : public IModuleInterface
{
public:

	FMultiServerConfigurationModule() {}
	virtual ~FMultiServerConfigurationModule() {}

	// IModuleInterface
	virtual bool SupportsDynamicReloading() override { return false; }
	virtual bool SupportsAutomaticShutdown() override { return false; }
};


