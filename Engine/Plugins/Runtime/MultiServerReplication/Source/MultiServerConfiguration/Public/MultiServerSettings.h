// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "MultiServerSettings.generated.h"

/**
 * Settings that control how a multi-server setup is configured and launched.
 * For example, to define 4 servers in a Game ini file it might look like this:
 * [/Script/MultiServerConfiguration.MultiServerSettings]
 * TotalNumServers=4
 */
UCLASS(Config=Game)
class MULTISERVERCONFIGURATION_API UMultiServerSettings : public UObject
{
	GENERATED_BODY()

private:
	// This is read from the LaunchMultiServer script (not code). We take the runtime value from -MultiServerNumServers parameter.
	UPROPERTY(Config)
	int32 TotalNumServers = 0;
};
