// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MultiServerReplication : ModuleRules
{
	public MultiServerReplication(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
			new string[] {
                "Core",
                "CoreUObject",
                "CoreOnline",
                "Engine",
				"NetCore",
				"MultiServerConfiguration",
				"IrisCore",
				"Sockets"
            }
        );

        PublicDependencyModuleNames.AddRange(
            new string[] {
				"OnlineSubsystemUtils"
			}
		);

        bAllowUETypesInNamespaces = true;
    }
}
