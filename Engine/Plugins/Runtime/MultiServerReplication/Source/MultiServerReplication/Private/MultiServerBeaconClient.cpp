// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiServerBeaconClient.h"
#include "MultiServerBeaconHostObject.h"
#include "MultiServerNetDriver.h"
#include "MultiServerNode.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/NetConnection.h"
#include "Engine/World.h"
#include "Misc/PackageName.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiServerBeaconClient)

int32 GMultiServerAllowRemoteObjectReferences = 1;
static FAutoConsoleVariableRef CVarMultiServerAllowRemoteObjectReferences(
	TEXT("multiserver.AllowRemoteObjectReferences"),
	GMultiServerAllowRemoteObjectReferences,
	TEXT("Whether references to UObjects are replicated as remote object references between servers"));

AMultiServerBeaconClient::AMultiServerBeaconClient() :
	Super()
{
	bOnlyRelevantToOwner = true;

	NetDriverName = FName(TEXT("MultiServerNetDriverClient"));
	NetDriverDefinitionName = FName(TEXT("MultiServerNetDriver"));

	// Allow this to tick on MultiServer nodes
	PrimaryActorTick.bAllowTickOnDedicatedServer = true;
}

FName AMultiServerBeaconClient::NetworkRemapPath(FName InPackageName, bool bReading)
{
	// For PIE Networking: remap the packagename to our local PIE packagename
	FString PackageNameStr = InPackageName.ToString();
	if (UNetConnection* Connection = GetNetConnection())
	{
		GEngine->NetworkRemapPath(Connection, PackageNameStr, bReading);
	}
	return FName(*PackageNameStr);
}

void AMultiServerBeaconClient::OnConnected()
{
	Super::OnConnected();

	UE_LOGF(LogMultiServerBeacon, Log, "MultiServer beacon connection established.");

	if (ensureMsgf(OwningNode, TEXT("No owning node")))
	{
		ServerSetRemotePeerId(OwningNode->GetLocalPeerId());
	}

	if (UNetConnection* Connection = GetNetConnection())
	{
		Connection->SetUnlimitedBunchSizeAllowed(true);
	}

	UWorld* World = GetWorld();
	if (World)
	{
		TArray<FUpdateLevelVisibilityLevelInfo> LevelVisibilities;
		for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
		{
			if (LevelStreaming)
			{
				const ULevel* Level = LevelStreaming->GetLoadedLevel();
				if (Level && Level->bIsVisible && !Level->bClientOnlyVisible)
				{
					FUpdateLevelVisibilityLevelInfo& LevelVisibility = *new(LevelVisibilities) FUpdateLevelVisibilityLevelInfo(Level, true);
					LevelVisibility.PackageName = NetworkRemapPath(LevelVisibility.PackageName, false);
				}
			}
		}
		if (LevelVisibilities.Num() > 0)
		{
			ServerUpdateMultipleLevelsVisibility(LevelVisibilities);
		}
	}

	OnLevelRemovedFromWorldHandle = FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &ThisClass::OnLevelRemovedFromWorld);
	OnLevelAddedToWorldHandle = FWorldDelegates::LevelAddedToWorld.AddUObject(this, &ThisClass::OnLevelAddedToWorld);
}

void AMultiServerBeaconClient::DestroyBeacon()
{
	FWorldDelegates::LevelRemovedFromWorld.Remove(OnLevelRemovedFromWorldHandle);
	FWorldDelegates::LevelAddedToWorld.Remove(OnLevelAddedToWorldHandle);

	Super::DestroyBeacon();
}

void AMultiServerBeaconClient::ConnectToServer(const FString& ConnectInfo)
{
	bool bSuccess = false;

	FOnlineBeaconClientInitParams Params;
	Params.URL = FURL(nullptr, *ConnectInfo, TRAVEL_Absolute);
	Params.bReplicateTransactionally = false;

	if (InitClient(Params))
	{
		bSuccess = true;
	}
	else
	{
		UE_LOGF(LogMultiServerBeacon, Warning, "ConnectToRelay: Failure to init client beacon with %ls.", *Params.URL.ToString());
	}

	if (!bSuccess)
	{
		AOnlineBeacon::OnFailure(EBeaconFailureReason::InitFailure, TEXTVIEW("beacon_multiserverclient_init_beacon_failure"));
	}
}

void AMultiServerBeaconClient::ClientPeerConnected_Implementation(const FString& NewRemotePeerId, AMultiServerBeaconClient* Beacon)
{
	RemotePeerId = NewRemotePeerId;

	if (OwningNode)
	{
		OwningNode->NotifyMultiServerConnected(OwningNode->GetLocalPeerId(), NewRemotePeerId, this);
	}
}

void AMultiServerBeaconClient::OnFailure()
{
	Super::OnFailure();
}

bool AMultiServerBeaconClient::InitBase()
{
	if (Super::InitBase() && NetDriver)
	{
		ensureMsgf(NetDriver->IsA<UMultiServerNetDriver>(), TEXT("Multi-server beacon NetDriver should be a subclass of UMultiServerNetDriver to function correctly. Check the NetDriverDefinition for MultiServerNetDriver."));

		NetDriver->SetUsingRemoteObjectReferences(!!UE_WITH_REMOTE_OBJECT_HANDLE && GMultiServerAllowRemoteObjectReferences);
		return true;
	}
	return false;
}

void AMultiServerBeaconClient::ServerUpdateLevelVisibility_Implementation(const FUpdateLevelVisibilityLevelInfo& LevelVisibility)
{
	if (GetConnectionState() == EBeaconConnectionState::Open)
	{
		if (UNetConnection* Connection = GetNetConnection())
		{
			if (Connection->Driver && Connection->Driver->IsServer())
			{
				FUpdateLevelVisibilityLevelInfo LevelVisibilityCopy = LevelVisibility;
				LevelVisibilityCopy.PackageName = NetworkRemapPath(LevelVisibility.PackageName, true);
				LevelVisibilityCopy.bSkipCloseOnError = true;
				
				Connection->UpdateLevelVisibility(LevelVisibilityCopy);
			}
		}
	}
}

bool AMultiServerBeaconClient::ServerUpdateLevelVisibility_Validate(const FUpdateLevelVisibilityLevelInfo& LevelVisibility)
{
	RPC_VALIDATE(LevelVisibility.PackageName.IsValid());

	FText Reason;

	if (!FPackageName::IsValidLongPackageName(LevelVisibility.PackageName.ToString(), true, &Reason))
	{
		UE_LOGF(LogMultiServerBeacon, Warning, "ServerUpdateLevelVisibility() Invalid package name: %ls (%ls)", *LevelVisibility.PackageName.ToString(), *Reason.ToString());
		return false;
	}

	return true;
}

void AMultiServerBeaconClient::ServerUpdateMultipleLevelsVisibility_Implementation(const TArray<FUpdateLevelVisibilityLevelInfo>& LevelVisibilities)
{
	for (const FUpdateLevelVisibilityLevelInfo& LevelVisibility : LevelVisibilities)
	{
		ServerUpdateLevelVisibility_Implementation(LevelVisibility);
	}
}

bool AMultiServerBeaconClient::ServerUpdateMultipleLevelsVisibility_Validate(const TArray<FUpdateLevelVisibilityLevelInfo>& LevelVisibilities)
{
	for (const FUpdateLevelVisibilityLevelInfo& LevelVisibility : LevelVisibilities)
	{
		if (!ServerUpdateLevelVisibility_Validate(LevelVisibility))
		{
			return false;
		}
	}

	return true;
}

void AMultiServerBeaconClient::ServerSetRemotePeerId_Implementation(const FString& NewRemoteId)
{
	UE_LOGF(LogMultiServerBeacon, Log, "ServerSetRemotePeerId: %ls %ls", *GetNameSafe(this), *NewRemoteId);
	RemotePeerId = NewRemoteId;

	if (OwningNode)
	{
		OwningNode->NotifyMultiServerConnected(OwningNode->GetLocalPeerId(), NewRemoteId, this);
	}
}

FString AMultiServerBeaconClient::GetRemotePeerId() const
{
	return RemotePeerId;
}

FString AMultiServerBeaconClient::GetLocalPeerId() const
{
	if (ensure(OwningNode))
	{
		return OwningNode->GetLocalPeerId();
	}

	return TEXT("OwningNode was nullptr");
}

bool AMultiServerBeaconClient::IsAuthorityBeacon() const
{
	return !GetNetDriver() || GetNetDriver()->ServerConnection == nullptr;
}

void AMultiServerBeaconClient::OnLevelRemovedFromWorld(ULevel* Level, UWorld* World)
{
	if (GetWorld() == World)
	{
		if (Level && !Level->bClientOnlyVisible)
		{
			FUpdateLevelVisibilityLevelInfo LevelVisibility(Level, false);
			LevelVisibility.PackageName = NetworkRemapPath(LevelVisibility.PackageName, false);

			ServerUpdateLevelVisibility(LevelVisibility);
		}
	}
}

void AMultiServerBeaconClient::OnLevelAddedToWorld(ULevel* Level, UWorld* World)
{
	if (GetWorld() == World)
	{
		if (Level && !Level->bClientOnlyVisible)
		{
			FUpdateLevelVisibilityLevelInfo LevelVisibility(Level, true);
			LevelVisibility.PackageName = NetworkRemapPath(LevelVisibility.PackageName, false);

			ServerUpdateLevelVisibility(LevelVisibility);
		}
	}
}