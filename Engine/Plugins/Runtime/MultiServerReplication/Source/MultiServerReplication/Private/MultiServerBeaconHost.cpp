// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiServerBeaconHost.h"
#include "MultiServerNetDriver.h"
#include "Misc/CommandLine.h"
#include "Net/DataChannel.h"
#include "Engine/NetConnection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiServerBeaconHost)

extern int32 GMultiServerAllowRemoteObjectReferences;

AMultiServerBeaconHost::AMultiServerBeaconHost(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	NetDriverName = FName(TEXT("MultiServerNetDriverHost"));
	NetDriverDefinitionName = FName(TEXT("MultiServerNetDriver"));
	MaxConnections = 100;
}

bool AMultiServerBeaconHost::InitHost()
{
	FURL URL(nullptr, TEXT(""), TRAVEL_Absolute);

	FString MultiServerBeaconHostAddr;
	if (FParse::Value(FCommandLine::Get(), TEXT("MultiServerHostAddr="), MultiServerBeaconHostAddr) && !MultiServerBeaconHostAddr.IsEmpty())
	{
		URL.AddOption(*FString::Printf(TEXT("multihome=%s"), *MultiServerBeaconHostAddr));
	}

	URL.Port = ListenPort;
	if(URL.Valid)
	{
		if (InitBase() && NetDriver)
		{
			ensureMsgf(NetDriver->IsA<UMultiServerNetDriver>(), TEXT("Multi-server beacon NetDriver should be a subclass of UMultiServerNetDriver to function correctly. Check the NetDriverDefinition for MultiServerNetDriver."));

			// This flag has to be set before InitListen and the ReplicationSystem is created if using Iris
			NetDriver->SetUsingRemoteObjectReferences(!!UE_WITH_REMOTE_OBJECT_HANDLE && GMultiServerAllowRemoteObjectReferences);
			NetDriver->SetReplicateTransactionally(false);

			FString Error;
			if (NetDriver->InitListen(this, URL, bReuseAddressAndPort, Error))
			{
				ListenPort = URL.Port;
				NetDriver->SetWorld(GetWorld());
				NetDriver->Notify = this;
				NetDriver->InitialConnectTimeout = BeaconConnectionInitialTimeout;
				NetDriver->ConnectionTimeout = BeaconConnectionTimeout;
				return true;
			}
			else
			{
				// error initializing the network stack...
				UE_LOGF(LogBeacon, Log, "AMultiServerBeaconHost::InitHost failed");
				OnFailure(EBeaconFailureReason::InitFailure, TEXTVIEW("beacon_multiserverhost_init_host_failure"));
			}
		}
	}

	return false;
}

bool AMultiServerBeaconHost::AtCapacity() const
{
	int32 Count = 0;

	for (UNetConnection* Connection : NetDriver->ClientConnections)
	{
		if (Connection && (Connection->GetConnectionState() != USOCK_Closed) && (Connection->OwningActor != nullptr))
		{
			++Count;
		}
	}

	return (Count >= MaxConnections);
}

void AMultiServerBeaconHost::NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch)
{
	if (NetDriver->ServerConnection == nullptr)
	{
		switch (MessageType)
		{
		case NMT_BeaconJoin:
			// if we are at capacity, intervene here
			if (AtCapacity())
			{
				FString ErrorMsg = TEXT("MultiServer beacon at capacity.");
				UE_LOGF(LogBeacon, Log, "%ls: %ls", *Connection->GetName(), *ErrorMsg);
				FNetControlMessage<NMT_Failure>::Send(Connection, ErrorMsg);
				Connection->FlushNet(true);
				Connection->Close();
				return;
			}
			break;
		}
	}

	Super::NotifyControlMessage(Connection, MessageType, Bunch);
}

