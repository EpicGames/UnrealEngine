// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconHost.h"
#include "MultiServerBeaconHost.generated.h"

/**
 * An online beacon that helps manage connecting to MultiServer networks, and replicating
 * metadata about the MultiServer network.
 */
UCLASS(transient, config=Engine)
class AMultiServerBeaconHost : public AOnlineBeaconHost
{
	GENERATED_BODY()

public:
	AMultiServerBeaconHost(const FObjectInitializer& ObjectInitializer);

	virtual bool InitHost() override;
	virtual void NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch) override;

	/** @return Whether or not this Node already has the maximum number of allowable connections. */
	virtual bool AtCapacity() const;

protected:
	UPROPERTY(config)
	int32 MaxConnections;
};
