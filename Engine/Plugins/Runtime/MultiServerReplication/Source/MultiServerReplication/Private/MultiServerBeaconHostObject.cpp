// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiServerBeaconHostObject.h"
#include "MultiServerBeaconClient.h"
#include "MultiServerNode.h"
#include "Engine/NetConnection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiServerBeaconHostObject)

DEFINE_LOG_CATEGORY(LogMultiServerBeacon);

AMultiServerBeaconHostObject::AMultiServerBeaconHostObject(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	SetClientBeaconActorClass(AMultiServerBeaconClient::StaticClass());
}

void AMultiServerBeaconHostObject::SetClientBeaconActorClass(TSubclassOf<AOnlineBeaconClient> InClientBeaconActorClass)
{
	ClientBeaconActorClass = InClientBeaconActorClass;
	BeaconTypeName = ClientBeaconActorClass->GetName();
}

void AMultiServerBeaconHostObject::OnClientConnected(AOnlineBeaconClient* NewClientActor, UNetConnection* ClientConnection)
{
	Super::OnClientConnected(NewClientActor, ClientConnection);

	UE_LOGF(LogMultiServerBeacon, Verbose, "AMultiServerBeaconHostObject::OnClientConnected: num client actors connected is %d", ClientActors.Num());

	ClientConnection->SetUnlimitedBunchSizeAllowed(true);

	if (AMultiServerBeaconClient* MultiServerClient = Cast<AMultiServerBeaconClient>(NewClientActor))
	{
		MultiServerClient->SetOwningNode(OwningNode);

		MultiServerClient->ClientPeerConnected(OwningNode->GetLocalPeerId(), MultiServerClient);
	}
}

void AMultiServerBeaconHostObject::NotifyClientDisconnected(AOnlineBeaconClient* LeavingClientActor)
{
	Super::NotifyClientDisconnected(LeavingClientActor);

	UE_LOGF(LogMultiServerBeacon, Verbose, "AMultiServerBeaconHostObject::NotifyClientDisconnected: num client actors connected is %d", ClientActors.Num());
}

