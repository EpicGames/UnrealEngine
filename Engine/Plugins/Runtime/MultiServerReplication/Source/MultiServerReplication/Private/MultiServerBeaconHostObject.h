// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconHostObject.h"
#include "MultiServerBeaconHostObject.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMultiServerBeacon, Log, All);

class AMultiServerBeaconClient;
class UMultiServerNode;

UCLASS(transient, config=Engine)
class AMultiServerBeaconHostObject : public AOnlineBeaconHostObject
{
	GENERATED_BODY()

public:
	AMultiServerBeaconHostObject(const FObjectInitializer& ObjectInitializer);

	void SetClientBeaconActorClass(TSubclassOf<AOnlineBeaconClient> InClientBeaconActorClass);
	void SetOwningNode(UMultiServerNode* InOwningNode) { OwningNode = InOwningNode; }

	virtual void OnClientConnected(AOnlineBeaconClient* NewClientActor, UNetConnection* ClientConnection) override;

	virtual void NotifyClientDisconnected(AOnlineBeaconClient* LeavingClientActor) override;

private:

	UPROPERTY()
	TObjectPtr<UMultiServerNode> OwningNode;
};
