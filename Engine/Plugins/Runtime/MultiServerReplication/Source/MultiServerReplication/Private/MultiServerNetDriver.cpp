// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiServerNetDriver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiServerNetDriver)

void UMultiServerNetDriver::SetWorld(UWorld* InWorld)
{
	Super::SetWorld(InWorld);

	// Unsubscribe from world tick events, these drivers are ticked manually outside of the normal world tick.
	UnregisterTickEvents(InWorld);
}