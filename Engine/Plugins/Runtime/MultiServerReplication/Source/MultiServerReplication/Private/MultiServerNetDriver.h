// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IpNetDriver.h"
#include "MultiServerNetDriver.generated.h"

// Multi-server nodes need to use UMultiServerNetDriver (or a subclass) because they control the ticking
// of their NetDrivers directly instead of letting the world tick them.
// 
// Use this MultiServerNetDriver as the DriverClassName for the MultiServerNetDriver definition in
// your project's NetDriverDefinitions engine config, for example:
// 
// [/Script/Engine.Engine]
// +NetDriverDefinitions=(DefName="MultiServerNetDriver", DriverClassName="/Script/MultiServerReplication.MultiServerNetDriver", DriverClassNameFallback="/Script/MultiServerReplication.MultiServerNetDriver")
// 
// This allows control over the timing of the Tick(Flush|Dispatch) and PostTick(Flush|Dispatch) functions
// to ensure they're always called as atomic units. Since MultiServer drivers might be ticked from within
// a NetDriver that's being ticked by the world, and the world ticks in passes (all netdrivers Tick, then
// all netdrivers PostTick), we could end up in a situation where a MultiServer driver has Ticked, and is
// Ticked again before the corresponding PostTick was called (if the world was allowed to tick
// the MultiServer drivers). This is not compatible with Iris replication.
// 
// We use a NetDriver subclass so we can override SetWorld and undo the normal world tick registration.
UCLASS(Transient)
class UMultiServerNetDriver : public UIpNetDriver
{
	GENERATED_BODY()

public:
	virtual void SetWorld(UWorld* InWorld) override;
};
