// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiServerNode.h"
#include "MultiServerBeaconHost.h"
#include "MultiServerBeaconHostObject.h"
#include "MultiServerBeaconClient.h"
#include "MultiServerPeerConnection.h"
#include "MultiServerReplicationTypes.h"
#include "Algo/Sort.h"
#include "Engine/NetConnection.h"
#include "Engine/World.h"
#include "Engine/NetDriver.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiServerNode)

void UMultiServerNode::ParseCommandLineIntoCreateParams(FMultiServerNodeCreateParams& InOutParams)
{
	FParse::Value(FCommandLine::Get(), TEXT("MultiServerLocalId="), InOutParams.LocalPeerId);
	FParse::Value(FCommandLine::Get(), TEXT("MultiServerListenPort="), InOutParams.ListenPort);

	FString PeerAddressesString;
	FParse::Value(FCommandLine::Get(), TEXT("MultiServerPeers="), PeerAddressesString, false);

	PeerAddressesString.ParseIntoArray(InOutParams.PeerAddresses, TEXT(","), true);

	FParse::Value(FCommandLine::Get(), TEXT("MultiServerNumServers="), InOutParams.NumServers);
	if (InOutParams.NumServers < 1)
	{
		InOutParams.NumServers = InOutParams.PeerAddresses.Num();
	}
}

UMultiServerNode::UMultiServerNode()
	: UObject()
{
	RetryConnectDelay = 0.5f;
	RetryConnectMaxDelay = 30.0f;
}

bool UMultiServerNode::AreAllServersConnected() const
{
	if (NumExpectedServers <= 1)
	{
		return true;
	}

	const bool bAllPeersAcknowledged = GetConnectionCount() >= GetNumExpectedConnections();
	return bAllPeersAcknowledged;
}

UMultiServerNode* UMultiServerNode::Create(const FMultiServerNodeCreateParams& Params)
{
	UMultiServerNode* NewNode = NewObject<UMultiServerNode>(Params.World);
	const bool bResult = NewNode->RegisterServer(Params);

	if (bResult)
	{
		NewNode->RegisterTickEvents();
		return NewNode;
	}
	else
	{
		// Is this the best way to clean up on failed registration?
		NewNode->MarkAsGarbage();
	}
	
	return nullptr;
}

void UMultiServerNode::BeginDestroy()
{
	UnregisterTickEvents();

	Super::BeginDestroy();
}

bool UMultiServerNode::RegisterServer(const FMultiServerNodeCreateParams& Params)
{
	if (Params.World == nullptr)
	{
		UE_LOGF(LogMultiServerReplication, Warning, "UMultiServerNode::RegisterServer: null world - failed to register.");
		return false;
	}

	if (Params.LocalPeerId.IsEmpty())
	{
		UE_LOGF(LogMultiServerReplication, Warning, "UMultiServerNode::RegisterServer: no MultiServerLocalId specified - required for multiserver to work properly.");
		return false;
	}

	NumExpectedServers = Params.NumServers;
	LocalPeerId = Params.LocalPeerId;
	UserBeaconClass = Params.UserBeaconClass;
	OnMultiServerConnectedEvent = Params.OnMultiServerConnected;

	if (Params.ListenPort == 0)
	{
		UE_LOGF(LogMultiServerReplication, Log, "UMultiServerNode::RegisterServer: no listen port specified, not listening.");
	}
	else
	{
		UE_LOGF(LogMultiServerReplication, Log, "UMultiServerNode::RegisterServer: setting up host beacon for %ls.", ToCStr(LocalPeerId));

		// Set up host beacon
		if (ensureMsgf(BeaconHost == nullptr, TEXT("UMultiServerNode::RegisterServer: BeaconHost already created.")))
		{
			// Always create a new beacon host, state will be determined in a moment
			BeaconHost = Params.World->SpawnActor<AMultiServerBeaconHost>(AMultiServerBeaconHost::StaticClass());
			check(BeaconHost);

			BeaconHost->ListenPort = Params.ListenPort;

			bool bBeaconInit = false;
			if (BeaconHost->InitHost())
			{ 
				BeaconHostObject = Params.World->SpawnActor<AMultiServerBeaconHostObject>(AMultiServerBeaconHostObject::StaticClass());
				check(BeaconHostObject);

				BeaconHostObject->SetClientBeaconActorClass(Params.UserBeaconClass);
				BeaconHostObject->SetOwningNode(this);

				BeaconHost->RegisterHost(BeaconHostObject);
				BeaconHost->PauseBeaconRequests(false);
			}
			else
			{
				UE_LOGF(LogMultiServerReplication, Warning, "Failed to init multiserver host beacon %ls", *BeaconHost->GetName());
				return false;
			}
		}
	}

	// If the -MultiServerPeers command-line option is set, start a client beacon for each one and connect to them.
	// (they are expected to be listening already)
	if (Params.PeerAddresses.Num() == 0)
	{
		UE_LOGF(LogMultiServerReplication, Log, "UMultiServerNode::RegisterServer: no peers specified, not connecting to any. LocalPeerId %ls", ToCStr(LocalPeerId));
	}
	else
	{
		// Assume every server wil receive the same list of addresses. Sort it so all servers can deterministically figure out who else they need to connect to.
		TArray<FString> SortedPeerAddresses = Params.PeerAddresses;
		Algo::Sort(SortedPeerAddresses);

		// If our local IP wasn't passed in, assume localhost.
		FString ListenIp = Params.ListenIp.IsEmpty() ? TEXT("127.0.0.1") : Params.ListenIp;

		// Find our own index in the peer list
		const int32 LocalIndex = SortedPeerAddresses.IndexOfByPredicate([&ListenIp, &Params](const FString& PeerAddress)
			{
				FURL PeerURL(nullptr, ToCStr(PeerAddress), ETravelType::TRAVEL_Absolute);
				return PeerURL.Host.Equals(ListenIp) && PeerURL.Port == Params.ListenPort;
			});

		if (LocalIndex == INDEX_NONE)
		{
			UE_LOGF(LogMultiServerReplication, Error, "Failed to find this local server instance in PeerAddresses list. Address: %ls:%hu. Not connecting to peers.", ToCStr(ListenIp), Params.ListenPort);
			return false;
		}

		// Balance the number of client vs. server connections to minimize the difference in number of unique
		// NetDrivers created by each peer. This helps minimize latency when iterating & processing each peer.

		// Rules, where n is the total number of peers:
		// If the number of peers is even:
		//  -The first of peers in SortedPeerAddresses connect to ceil((n - 1) / 2) other servers.
		//  -The second half of peers in SortedPeerAddresses connect to floor((n - 1) / 2) other servers.
		//
		// If the number of peers is odd:
		//  -All peers connect to (n - 1) / 2 servers.
		//
		// All connections are made starting with the index directly above the local peer's own in the sorted list,
		// wrapping around.
		//
		// For example, when there are 4 peers, each one would make connections as follows:
		// 1 -> 2 3
		// 2 -> 3 4
		// 3 -> 4
		// 4 -> 1
		// 
		// Or with 7 peers:
		// 1 -> 2 3 4
		// 2 -> 3 4 5
		// 3 -> 4 5 6
		// 4 -> 5 6 7
		// 5 -> 6 7 1
		// 6 -> 7 1 2
		// 7 -> 1 2 3
		//
		const int32 CeilConnections = FMath::CeilToInt((Params.PeerAddresses.Num() - 1) / 2.0f);
		const int32 FloorConnections = FMath::FloorToInt((Params.PeerAddresses.Num() - 1) / 2.0f);

		// When the number of total peers is even, peers in the first half of the list will connect to one more
		// peer than those in the second half of the list.
		//
		// When the number of total peers is odd, all peers will connect to the same number of other peers.
		const int32 NumToConnectTo = LocalIndex < (Params.PeerAddresses.Num() / 2) ? CeilConnections : FloorConnections;

		UE_LOGF(LogMultiServerReplication, Log, "Connecting to %d other peers.", NumToConnectTo);

		for (int32 RemotePeerCount = 0; RemotePeerCount < NumToConnectTo; ++RemotePeerCount)
		{
			const int32 RemotePeerIndex = (LocalIndex + RemotePeerCount + 1) % Params.PeerAddresses.Num();
			if (!ensure(SortedPeerAddresses.IsValidIndex(RemotePeerIndex)))
			{
				UE_LOGF(LogMultiServerReplication, Error, "Peer index %d is out of range. Number of peer addresses: %d.", RemotePeerIndex, Params.PeerAddresses.Num());
				continue;
			}

			const FString& PeerAddress = SortedPeerAddresses[RemotePeerIndex];

			UE_LOGF(LogMultiServerReplication, Log, "Connecting to peer at address %ls.", ToCStr(PeerAddress));
			if (!PeerAddress.IsEmpty())
			{
				UMultiServerPeerConnection* Peer = NewObject<UMultiServerPeerConnection>(this);
				Peer->SetOwningNode(this);
				Peer->SetRemoteAddress(PeerAddress);
				Peer->SetLocalPeerId(LocalPeerId);
				Peer->InitClientBeacon();
				PeerConnections.Add(Peer);
			}
		}
	}

	if (NumExpectedServers <= 1)
	{
		Params.World->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateWeakLambda(this, [this]()
			{
				UE_AUTORTFM_ONCOMMIT(this)
				{
					BroadcastAllServersConnected();
				};
			}));
	}

	return true;
}

void UMultiServerNode::RegisterTickEvents()
{
	UWorld* World = GetWorld();
	if (World)
	{
		TickDispatchDelegateHandle = World->OnTickDispatch().AddUObject(this, &UMultiServerNode::InternalTickDispatch);
		TickFlushDelegateHandle = World->OnTickFlush().AddUObject(this, &UMultiServerNode::InternalTickFlush);
	}
}

void UMultiServerNode::UnregisterTickEvents()
{
	UWorld* World = GetWorld();
	if (World)
	{
		World->OnTickDispatch().Remove(TickDispatchDelegateHandle);
		World->OnTickFlush().Remove(TickFlushDelegateHandle);
	}
}

void UMultiServerNode::InternalTickDispatch(float DeltaSeconds)
{
	ForEachNetDriver([DeltaSeconds](UNetDriver* NetDriver)
		{
			if (NetDriver->GetWorld())
			{
				NetDriver->TickDispatch(DeltaSeconds);
				NetDriver->PostTickDispatch();
			}
		});
}

void UMultiServerNode::InternalTickFlush(float DeltaSeconds)
{
	ForEachNetDriver([DeltaSeconds](UNetDriver* NetDriver)
		{
			if (NetDriver->GetWorld())
			{
				NetDriver->TickFlush(DeltaSeconds);
				NetDriver->PostTickFlush();
			}
		});
}

AMultiServerBeaconClient* UMultiServerNode::GetBeaconClientForRemotePeer(FStringView RemotePeerId) const
{
	// See if we are the host of the target server
	UNetDriver* HostNetDriver = BeaconHost ? BeaconHost->GetNetDriver() : nullptr;
	if (HostNetDriver)
	{
		for (UNetConnection* ClientConnection : HostNetDriver->ClientConnections)
		{
			AMultiServerBeaconClient* BeaconClient = Cast<AMultiServerBeaconClient>(BeaconHost->GetClientActor(ClientConnection));
			if (BeaconClient)
			{
				if (RemotePeerId.Equals(BeaconClient->GetRemotePeerId(), ESearchCase::IgnoreCase))
				{
					return BeaconClient;
				}
			}
		}
	}

	// See if we are a client of the target server
	for (UMultiServerPeerConnection* Peer : PeerConnections)
	{
		AMultiServerBeaconClient* Beacon = Peer->BeaconClient;
		if (Beacon)
		{
			if (RemotePeerId.Equals(Beacon->GetRemotePeerId(), ESearchCase::IgnoreCase))
			{
				return Beacon;
			}
		}
	}

	return nullptr;
}

AMultiServerBeaconClient* UMultiServerNode::GetBeaconClientForURL(const FString& InURL) const
{
	FURL URL(nullptr, ToCStr(InURL), TRAVEL_Absolute);

	// See if we are the host of the target server
	UNetDriver* HostNetDriver = BeaconHost ? BeaconHost->GetNetDriver() : nullptr;
	if (HostNetDriver)
	{
		for (UNetConnection* ClientConnection : HostNetDriver->ClientConnections)
		{
			if (ClientConnection &&
				ClientConnection->URL.Host.Equals(URL.Host, ESearchCase::IgnoreCase) &&
				ClientConnection->URL.Port == URL.Port)
			{
				return Cast<AMultiServerBeaconClient>(BeaconHost->GetClientActor(ClientConnection));
			}
		}
	}

	// See if we are a client of the target server
	for (UMultiServerPeerConnection* Peer : PeerConnections)
	{
		AMultiServerBeaconClient* Beacon = Peer->BeaconClient;
		if (Beacon)
		{
			UNetConnection* ClientConnection = Beacon->GetNetConnection();
			if (ClientConnection &&
				ClientConnection->URL.Host.Equals(URL.Host, ESearchCase::IgnoreCase) &&
				ClientConnection->URL.Port == URL.Port)
			{
				return Beacon;
			}
		}
	}

	return nullptr;
}

void UMultiServerNode::ForEachBeaconClient(TFunctionRef<void(AMultiServerBeaconClient*)> Operation) const
{
	UNetDriver* HostNetDriver = BeaconHost ? BeaconHost->GetNetDriver() : nullptr;
	if (HostNetDriver)
	{
		for (UNetConnection* ClientConnection : HostNetDriver->ClientConnections)
		{
			AMultiServerBeaconClient* BeaconClient = Cast<AMultiServerBeaconClient>(BeaconHost->GetClientActor(ClientConnection));
			if (BeaconClient)
			{
				Operation(BeaconClient);
			}
		}
	}
	for (UMultiServerPeerConnection* Peer : PeerConnections)
	{
		AMultiServerBeaconClient* BeaconClient = Peer->BeaconClient;
		if (BeaconClient)
		{
			Operation(BeaconClient);
		}
	}
}

void UMultiServerNode::ForEachNetDriver(TFunctionRef<void(UNetDriver*)> Operation) const
{
	TSet<UNetDriver*> UniqueNetDrivers;

	UNetDriver* HostNetDriver = BeaconHost ? BeaconHost->GetNetDriver() : nullptr;
	if (HostNetDriver)
	{
		UniqueNetDrivers.Add(HostNetDriver);
	}

	for (int32 PeerIndex = 0; PeerIndex < PeerConnections.Num(); PeerIndex++)
	{
		UMultiServerPeerConnection* Peer = PeerConnections[PeerIndex];
		if (AMultiServerBeaconClient* BeaconClient = Peer->BeaconClient)
		{
			if (UNetConnection* Connection = BeaconClient->GetNetConnection())
			{
				if (UNetDriver* NetDriver = Connection->GetDriver())
				{
					UniqueNetDrivers.Add(NetDriver);
				}
			}
		}
	}

	for (UNetDriver* NetDriver : UniqueNetDrivers)
	{
		Operation(NetDriver);
	}
}

uint32 UMultiServerNode::GetConnectionCount() const
{
	uint32 ConnectionCount = 0;

	const_cast<UMultiServerNode*>(this)->ForEachBeaconClient([&ConnectionCount](AMultiServerBeaconClient* BeaconClient)
	{
		if (!BeaconClient->GetRemotePeerId().IsEmpty())
		{
			ConnectionCount++;
		}
	});

	return ConnectionCount;
}

void UMultiServerNode::NotifyMultiServerConnected(const FString& LocalServerId, const FString& ConnectedRemoteServerId, AMultiServerBeaconClient* ConnectionToServer)
{
	OnMultiServerConnectedEvent.Broadcast(LocalServerId, ConnectedRemoteServerId, ConnectionToServer);
}
