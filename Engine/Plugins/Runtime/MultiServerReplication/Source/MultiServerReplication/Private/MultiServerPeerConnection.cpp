// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiServerPeerConnection.h"
#include "MultiServerReplicationTypes.h"
#include "MultiServerNode.h"
#include "MultiServerBeaconClient.h"
#include "Engine/World.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiServerPeerConnection)

UMultiServerPeerConnection::UMultiServerPeerConnection()
	: Super()
{
	
}

void UMultiServerPeerConnection::InitClientBeacon()
{
	DestroyClientBeacon();

	BeaconClient = GetWorld()->SpawnActor<AMultiServerBeaconClient>(OwningNode->GetUserBeaconClass());
	if (BeaconClient)
	{
		UE_LOGF(LogMultiServerReplication, Verbose, "Created multiserver client beacon %ls.", *BeaconClient->GetName());

		BeaconClient->OnHostConnectionFailure().BindUObject(this, &ThisClass::OnBeaconConnectionFailure);
		BeaconClient->SetOwningNode(OwningNode);

		if (!RemoteAddress.IsEmpty())
		{
			BeaconClient->ConnectToServer(RemoteAddress);
		}
		else
		{
			UE_LOGF(LogMultiServerReplication, Verbose, "Failed to get connection info for client beacon %ls", *BeaconClient->GetName());
			OnBeaconConnectionFailure();
		}
	}
	else
	{
		UE_LOGF(LogMultiServerReplication, Warning, "Failed to init MultiServer client beacon for %ls", ToCStr(RemoteAddress));
		OnBeaconConnectionFailure();
	}
}

void UMultiServerPeerConnection::DestroyClientBeacon()
{
	ClearConnectRetryTimer();

	if (BeaconClient)
	{
		UE_LOGF(LogMultiServerReplication, Verbose, "Destroying MultiServer beacon client.");

		BeaconClient->OnHostConnectionFailure().Unbind();

		BeaconClient->DestroyBeacon();
		BeaconClient = nullptr;
	}
}

void UMultiServerPeerConnection::ClearConnectRetryTimer()
{
	if (UWorld* World = GetWorld())
	{
		World->GetTimerManager().ClearTimer(ConnectRetryTimerHandle);
	}
}

float UMultiServerPeerConnection::GetRetryDelay()
{
	// Retry delay that increases exponentially
	++ConnectAttemptNum;

	// Randomize the initial delay
	float RetryDelay = FMath::RandRange(0.1f, OwningNode->GetRetryConnectDelay());

	if (ConnectAttemptNum > 1)
	{
		RetryDelay += OwningNode->GetRetryConnectDelay() * FMath::Pow(2.f, ConnectAttemptNum - 1);
	}

	RetryDelay = FMath::Min(RetryDelay, OwningNode->GetRetryConnectMaxDelay());

	return RetryDelay;
}

void UMultiServerPeerConnection::OnBeaconConnectionFailure()
{
	UE_LOGF(LogMultiServerReplication, Log, "MultiServer beacon connection failed.");

	DestroyClientBeacon();

	float Delay = GetRetryDelay();

	UE_LOGF(LogMultiServerReplication, Log, "MultiServer peer connect retry in %.2f seconds, attempt #%d", Delay, ConnectAttemptNum);

	UWorld* World = GetWorld();
	if (World)
	{
		World->GetTimerManager().SetTimer(ConnectRetryTimerHandle, FTimerDelegate::CreateWeakLambda(this, [this]()
			{
				InitClientBeacon();
			}), Delay, false);
	}
}