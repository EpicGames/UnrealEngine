// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "Engine/TimerHandle.h"
#include "Templates/UnrealTemplate.h"
#include "MultiServerPeerConnection.generated.h"

class AMultiServerBeaconClient;
class UMultiServerNode;

/** Net connection class specific to MultiServer Networking. */
UCLASS(Transient)
class UMultiServerPeerConnection : public UObject
{
	GENERATED_BODY()

public:
	UMultiServerPeerConnection();

	void InitClientBeacon();
	void DestroyClientBeacon();
	void ClearConnectRetryTimer();

	float GetRetryDelay();

	void OnBeaconConnectionFailure();

	UPROPERTY()
	TObjectPtr<AMultiServerBeaconClient> BeaconClient;

	void SetOwningNode(UMultiServerNode* InOwningNode) { OwningNode = InOwningNode; }
	void SetRemoteAddress(FString InRemoteAddress) { RemoteAddress = MoveTemp(InRemoteAddress); }
	void SetLocalPeerId(FString InLocalPeerId) { LocalPeerId = InLocalPeerId; }

private:
	int ConnectAttemptNum = 0;

	FTimerHandle ConnectRetryTimerHandle;

	UPROPERTY()
	TObjectPtr<UMultiServerNode> OwningNode;

	FString RemoteAddress;
	FString LocalPeerId;
};
