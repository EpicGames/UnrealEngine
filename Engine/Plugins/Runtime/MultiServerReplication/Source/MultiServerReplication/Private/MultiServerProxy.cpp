// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiServerProxy.h"
#include "UnrealEngine.h"
#include "EngineUtils.h"
#include "Engine/LocalPlayer.h"
#include "Engine/GameInstance.h"
#include "Engine/NetworkObjectList.h"
#include "GameFramework/GameModeBase.h"
#include "Misc/CommandLine.h"
#include "Net/DataChannel.h"
#include "Net/Core/Misc/NetSubObjectRegistry.h"
#include "Net/NetworkMetricsDatabase.h"
#include "Net/NetworkMetricsDefs.h"
#include "Engine/LevelStreaming.h"
#include "GameFramework/UpdateLevelVisibilityLevelInfo.h"

#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"

DEFINE_LOG_CATEGORY_STATIC(LogNetProxy, Log, All);

namespace UE::MultiServerProxy::Private
{
	static void CopyParmsFromFunction(TArray<uint8>& DestParms, const void* SrcData, int32 Size, const UFunction* Func)
	{
		DestParms.SetNumZeroed(Size);
		if (Func)
		{
			for (TFieldIterator<FProperty> It(Func); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
			{
				It->CopyCompleteValue_InContainer(DestParms.GetData(), SrcData);
			}
		}
	}

	FPendingBroadcastRPC::FPendingBroadcastRPC(const UFunction* InFunction, const void* InParms, const UObject* InSubObject)
		: Function(InFunction)
		, SubObject(InSubObject)
	{
		CopyParmsFromFunction(Parms, InParms, InFunction->ParmsSize, InFunction);
	}

	FPendingBroadcastRPC::FPendingBroadcastRPC(const FPendingBroadcastRPC& Other)
		: Function(Other.Function)
		, SubObject(Other.SubObject)
	{
		if (Other.Parms.Num() > 0)
		{
			CopyParmsFromFunction(Parms, Other.Parms.GetData(), Other.Parms.Num(), Function.ResolveObjectPtr());
		}
	}

	FPendingBroadcastRPC& FPendingBroadcastRPC::operator=(const FPendingBroadcastRPC& Other)
	{
		if (this != &Other)
		{
			DestroyParms();
			Function = Other.Function;
			SubObject = Other.SubObject;
			if (Other.Parms.Num() > 0)
			{
				CopyParmsFromFunction(Parms, Other.Parms.GetData(), Other.Parms.Num(), Function.ResolveObjectPtr());
			}
		}
		return *this;
	}

	FPendingBroadcastRPC::~FPendingBroadcastRPC()
	{
		DestroyParms();
	}

	void FPendingBroadcastRPC::DestroyParms()
	{
		if (Parms.Num() == 0)
		{
			return;
		}

		const UFunction* Func = Function.ResolveObjectPtrEvenIfGarbage();
		if (Func)
		{
			for (TFieldIterator<FProperty> It(Func); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
			{
				It->DestroyValue_InContainer(Parms.GetData());
			}
		}

		Parms.Reset();
	}

	FString ConnectionToString(const UNetConnection* Connection)
	{
		if (Connection)
		{
			return FString::Printf(TEXT("%s:%s"), *GetNameSafe(Connection->GetDriver()), *GetNameSafe(Connection));
		}
		else
		{
			return FString::Printf(TEXT("None:None"));
		}
	}

	FString PlayerControllerToString(const APlayerController* PlayerController)
	{
		if (PlayerController)
		{
			return FString::Printf(TEXT("%s:%s"), *GetNameSafe(PlayerController), *GetNameSafe(PlayerController->GetClass()));
		}
		else
		{
			return FString::Printf(TEXT("None:None"));
		}
	}

	float NonPrimarySetViewTargetIntervalSec = 1.0;
	static FAutoConsoleVariableRef CVarNonPrimarySetViewTargetIntervalSec(
		TEXT("net.proxy.NonPrimarySetViewTargetInterval"),
		NonPrimarySetViewTargetIntervalSec,
		TEXT("The interval, in seconds, between updating the view target position on non-primary game servers."));
	
	bool bEnableDisconnectionSupport = true;
	static FAutoConsoleVariableRef CVarEnableDisconnectionSupport(
		TEXT("net.proxy.EnableDisconnectionSupport"),
		bEnableDisconnectionSupport,
		TEXT("When true, when a client disconnects from the proxy it will close the connection to the backend server."));

	bool bEnableParentConnectionReplication = false;
	static FAutoConsoleVariableRef CVarEnableParentConnectionReplication(
		TEXT("net.proxy.EnableParentConnectionReplication"),
		bEnableParentConnectionReplication,
		TEXT("When true, the proxy will configure the parent connection to the backend game server to replicate state."));

	/**
	 * The proxy behaves as both a client and a server but there are values in APlayerController that are set based on it being on a
	 * client or a server.
	 *
	 * By default, these values in the player controller are set as if they were on a client because they were replicated to the client
	 * from the game server, but this class can be used to temporarily configure these values as if they were on a server.
	 */
	struct FScopedPlayerControllerPlayerSwap
	{
		FScopedPlayerControllerPlayerSwap(APlayerController* InPlayerController, UProxyNetDriver* InProxyNetDriver)
			: PlayerController(InPlayerController)
		{
			if (PlayerController && InProxyNetDriver)
			{
				if (ULocalPlayer* OwningPlayer = Cast<ULocalPlayer>(PlayerController->Player))
				{
					if (UNetConnection* ProxyConnection = InProxyNetDriver->GetProxyConnectionForPlayer(OwningPlayer))
					{
						OriginalPlayer = PlayerController->Player;
						PlayerController->Player = ProxyConnection;
					}
				}
			}
		}

		~FScopedPlayerControllerPlayerSwap()
		{
			if (OriginalPlayer)
			{
				PlayerController->Player = OriginalPlayer;
			}
		}

		FScopedPlayerControllerPlayerSwap(const FScopedPlayerControllerPlayerSwap&) = delete;
		FScopedPlayerControllerPlayerSwap& operator=(const FScopedPlayerControllerPlayerSwap&) = delete;
		FScopedPlayerControllerPlayerSwap(FScopedPlayerControllerPlayerSwap&&) = delete;
		FScopedPlayerControllerPlayerSwap& operator=(FScopedPlayerControllerPlayerSwap&&) = delete;

	private:
		APlayerController* PlayerController = nullptr;
		TObjectPtr<UPlayer> OriginalPlayer;
	};
}

namespace UE::MultiServerProxy::Metrics
{
	// The frame rate (FPS) and associated frame time in milliseconds of the process.
	const FName FPS("FPS");
	const FName FrameTimeMS("FrameTimeMS");
	
	// The number of RPCs forwarded from the proxy client to a primary game server.
	const FName NumForwardClientRPC("NumForwardClientRPC");

	// The number of RPCs forwarded from a game server to a proxy connection.
	const FName NumForwardBackendRPC("NumForwardBackendRPC");
	
	// The size of the UProxyNetDriver::InternalRoutes array which represents a path from each proxy
	// client to each game server.
	const FName NumProxyRoutes("NumProxyRoutes");
	
	// Aggregated InRate from all NetDrivers used to connect to a backend game server.
	const FName AggregatedBackendInRate("AggregatedBackendInRate");
	
	// Aggregated OutRate from all NetDrivers used to connect to a backend game server.
	const FName AggregatedBackendOutRate("AggregatedBackendOutRate");

	// The number of player controller's re-assigned on backend game servers.
	const FName NumReassignPlayerController("NumReassignPlayerController");
}

FProxyNetGUIDCache::FProxyNetGUIDCache(UNetDriver* NetDriver) : FNetGUIDCache(NetDriver)
{

}

FNetworkGUID FProxyNetGUIDCache::AssignNewNetGUID_Server(UObject* Object)
{
	// The proxy's NetGUID cache does not assign NetGUIDs directly but rather looks up the corresponding
	// NetGUID for the object in the shared backend NetGUID cache - this cache is populated using state
	// replicated from the backend game servers which, in turn, have the authority to assign NetGUIDs.
	//
	// What this means is that this function can potentially return an invalid NetGUID or default NetGUID in 
	// certain cases where the object is not in the shared backend NetGUID cache.
	return LookupNetGUIDFromBackendCache(Object);
}

FNetworkGUID FProxyNetGUIDCache::AssignNewNetGUIDFromPath_Server(const FString& PathName, UObject* ObjOuter, UClass* ObjClass)
{
	ensureMsgf(0, TEXT("UProxyNetDriver does not support assinging NetGUIDs from a path."));
	return FNetworkGUID::GetDefault();
}

FNetworkGUID FProxyNetGUIDCache::LookupNetGUIDFromBackendCache(UObject *Object)
{
	UProxyNetDriver* ProxyNetDriver = Cast<UProxyNetDriver>(Driver);
	
	if (ensureMsgf(ProxyNetDriver, TEXT("FProxyNetGUIDCache is only intended to be used by UProxyNetDriver")))
	{
		TSharedPtr<FNetGUIDCache> BackendGuidCache = ProxyNetDriver->GetSharedBackendNetGuidCache();
		if (ensure(BackendGuidCache))
		{
			FNetworkGUID MatchingNetGUID = BackendGuidCache->GetNetGUID(Object);
			if (MatchingNetGUID.IsValid())
			{
				// Catch cases where the NetGUID is already in the proxy cache.
				ensure(!GetNetGUID(Object).IsValid());

				UE_LOGF(LogNetProxy, Verbose, "NetGUID: Registering NetGUID %ls for object %ls found in the shared NetGUID cache.", *MatchingNetGUID.ToString(), *Object->GetName());
				RegisterNetGUID_Server(MatchingNetGUID, Object);
				return MatchingNetGUID;
			}
			else
			{
				// This function is using UObject::IsNameStableForNetworking() instead of FNetGUIDCache::IsDynamicObject(), because IsDynamicObject() considers if the outer object 
				// has an absolute path which might not be the case if Object is a static object component (e.g. render mesh) for a dynamic object (e.g. pawns and characters).
				if (ensureMsgf(Object->IsNameStableForNetworking(), TEXT("NetGUID: Failed to find a NetGUID for dynamic object %s in the shared NetGUID cache."), *Object->GetName()))
				{
					// This can occur when the proxy receives an RPC from the client with an RPC argument that is a static object reference. In this case the proxy 
					// hasn't heard about this object from the game server yet so it wouldn't know the NetGUID. Since static object references use a path rather than
					// a NetGUID the object will still be successfully found by the package map and forwarded to the game server.
					UE_LOGF(LogNetProxy, Verbose, "NetGUID: Static object %ls in not in the shared NetGUID cache yet.", *Object->GetName());
					return FNetworkGUID::GetDefault();
				}
			}
		}
	}

	return FNetworkGUID();
}

FProxyBackendNetGUIDCache::FProxyBackendNetGUIDCache(UNetDriver* NetDriver) : FNetGUIDCache(NetDriver)
{

}

bool FProxyBackendNetGUIDCache::IsNetGUIDAuthority() const
{
	// The backend NetGUID cache will always not be a GUID authority (i.e. it won't assign new GUIDs). This
	// is required because the default implementation queries the associated NetDriver which will be UProxyNetDriver
	// and a listening server (i.e. this function would return true).
	return false;
}

void UGameServerNotify::NotifyAcceptedConnection(class UNetConnection* Connection)
{

}

EAcceptConnection::Type UGameServerNotify::NotifyAcceptingConnection()
{
	return EAcceptConnection::Accept;
}

bool UGameServerNotify::NotifyAcceptingChannel(class UChannel* Channel)
{
	return true;
}

void UGameServerNotify::NotifyControlMessage(UNetConnection* GameServerConnection, uint8 MessageType, class FInBunch& Bunch)
{
	// NMT_CloseChildConnection is handled by UWorld::NotifyControlMessage which is bypassed by UGameServerNotify because it's
	// a sub-class of UPendingNetGame. For this reason these messages have to be explicitly handled below otherwise child connections 
	// to the game server will not close correctly.
	if (MessageType == NMT_CloseChildConnection)
	{
		int32 NetPlayerIndexToClose;
		if (FNetControlMessage<NMT_CloseChildConnection>::Receive(Bunch, NetPlayerIndexToClose))
		{
			UE_LOGF(LogNet, Log, "Received NMT_CloseChildConnection with NetPlayerIndex %d", NetPlayerIndexToClose);

			if (UChildConnection* ChildConnection = GameServerConnection->GetChildConnectionWithNetPlayerIndex(NetPlayerIndexToClose))
			{
				// The client shouldn't destroy actors as it will be done by the server.
				ChildConnection->CloseAndRemoveChild(UE::Net::ECloseChildFlags::None);
			}
			else
			{
				UE_LOGF(LogNet, Warning, "Unable to find a child connection matching NetPlayerIndex %d", NetPlayerIndexToClose);
			}
		}
	}
	else
	{
		Super::NotifyControlMessage(GameServerConnection, MessageType, Bunch);
	}

	if (MessageType == NMT_Welcome)
	{
		// The default implementation of UPendingNetGame will only send the join request to the server when the level
		// has loaded. Since the proxy is not currently dependent on loading levels we just send the join request 
		// when receiving the welcome message to shortcut this logic.
		SendJoinWithFlags(Flags);
	}
}

void UGameServerNotify::SetProxyNetDriver(TObjectPtr<class UProxyNetDriver> InProxyNetDriver)
{
	ProxyNetDriver = InProxyNetDriver;
}

void UGameServerNotify::SetFirstPlayer(TObjectPtr<ULocalPlayer> Player)
{
	FirstPlayer = Player;
}

ULocalPlayer* UGameServerNotify::GetFirstGamePlayer()
{
	return FirstPlayer;
}

void UProxyListenerNotify::SetProxyNetDriver(TObjectPtr<class UProxyNetDriver> InProxyNetDriver)
{
	ProxyNetDriver = InProxyNetDriver;
}

void UProxyListenerNotify::NotifyAcceptedConnection(class UNetConnection* Connection)
{

}

EAcceptConnection::Type UProxyListenerNotify::NotifyAcceptingConnection()
{
	return EAcceptConnection::Accept;
}

bool UProxyListenerNotify::NotifyAcceptingChannel(class UChannel* Channel)
{
	return true;
}

void UProxyListenerNotify::NotifyControlMessage(UNetConnection* ProxyConnection, uint8 MessageType, class FInBunch& Bunch)
{
	check(ProxyNetDriver);

	// The NMT_Join message received by a proxy connection should trigger a connection to the backend game servers.
	if (MessageType == NMT_Join)
	{
		if (ProxyNetDriver->bRandomizePrimaryGameServerForNextClient)
		{
			ProxyNetDriver->PrimaryGameServerForNextClient = FMath::RandHelper(ProxyNetDriver->GetGameServerConnectionCount());
			ProxyNetDriver->bRandomizePrimaryGameServerForNextClient = false;
		}

		// For now, the primary game server is always the first registered game server.
		for (int32 Index = 0; Index < ProxyNetDriver->GetGameServerConnectionCount(); Index++)
		{
			UE::Net::EJoinFlags Flags = UE::Net::EJoinFlags::NoPawn;
			if (Index == ProxyNetDriver->PrimaryGameServerForNextClient)
			{
				EnumRemoveFlags(Flags, UE::Net::EJoinFlags::NoPawn);
			}
			ConnectToGameServer(ProxyConnection, ProxyNetDriver->PrimaryGameServerForNextClient, ProxyNetDriver->GetGameServerConnection(Index), Flags);
		}

		if (ProxyNetDriver->bCyclePrimaryGameServer)
		{
			ProxyNetDriver->PrimaryGameServerForNextClient = (ProxyNetDriver->PrimaryGameServerForNextClient + 1) % ProxyNetDriver->GetGameServerConnectionCount();
		}
	}
	else
	{
		// Forward all other connection messages onto the existing handshake logic.
		ProxyNetDriver->GetWorld()->NotifyControlMessage(ProxyConnection, MessageType, Bunch);
	}
}

void UProxyListenerNotify::ConnectToGameServer(UNetConnection* ProxyConnection, 
											   int32 GameServerConnectionStateIndex,
											   FGameServerConnectionState* GameServerConnectionState,
											   UE::Net::EJoinFlags Flags)
{
	// If this is the first connection to the game server, instantiate the backend network driver that will manage all
	// connections from proxy connections to that server.
	if (GameServerConnectionState->NetDriver == nullptr)
	{
		// Acts as a unique identifier for dependency NetDrivers.
		static int32 GameServerDriverId = 0;

		GameServerConnectionState->World = ProxyNetDriver->GetWorld();
		GameServerConnectionState->NetDriverName = FName(*FString::Printf(TEXT("ProxyToGameServer-%d"), GameServerDriverId++));

		GEngine->CreateNamedNetDriver(ProxyNetDriver->GetWorld(), GameServerConnectionState->NetDriverName, "ProxyBackendNetDriver");
		GameServerConnectionState->NetDriver = Cast<UProxyBackendNetDriver>(GEngine->FindNamedNetDriver(GameServerConnectionState->World, GameServerConnectionState->NetDriverName));

		ensure(GameServerConnectionState->NetDriver);
		
		GameServerConnectionState->NetDriver->SetWorld(ProxyNetDriver->GetWorld());
		GameServerConnectionState->NetDriver->SetProxyNetDriver(ProxyNetDriver);
		GameServerConnectionState->NetDriver->GuidCache = ProxyNetDriver->GetSharedBackendNetGuidCache();

		UE_LOGF(LogNetProxy, Log, "Created a Proxy Game Server NetDriver (name=%ls, url=%ls)", *GameServerConnectionState->NetDriver->GetName(), *GameServerConnectionState->GameServerURL.ToString());
	}

	UNetConnection* GameServerConnection = GameServerConnectionState->NetDriver->ServerConnection;
	const bool bIsFirstGameServerConnection = (GameServerConnection == nullptr);
	if (bIsFirstGameServerConnection)
	{
		const uint32 ParentClientHandshakeId = ProxyNetDriver->GetNextClientHandshakeId();

		// The parent connection will not be associated with a player on the proxy and only exists so to use the control channel
		// and to allow all players (child connections) to be closed without having to worry about closing a parent connection and
		// promoting a child connection to parent connection.

		const FPlatformUserId DummyClientId = FPlatformUserId::CreateFromInternalId(ProxyNetDriver->GetNextGameServerClientId());
		ULocalPlayer* DummyPlayer = NewObject<ULocalPlayer>(GEngine, GEngine->LocalPlayerClass);
		DummyPlayer->SetConnectionIdentifier(ParentClientHandshakeId);
		ProxyNetDriver->GetWorld()->GetGameInstance()->UGameInstance::AddLocalPlayer(DummyPlayer, DummyClientId);

		FURL URL = GameServerConnectionState->GameServerURL;
		
		const FString GameLoginOptions = DummyPlayer->GetGameLoginOptions();
		if (!GameLoginOptions.IsEmpty())
		{
			URL.AddOption(*GameLoginOptions);
		}

		URL.AddOption(*FString::Printf(TEXT("HandshakeId=%u"), ParentClientHandshakeId));

		// Maybe this should be a control message since it changes the server setting (it's global driver setting).
		URL.AddOption(TEXT("Proxy"));

		// Start the connection flow to the game server.
		GameServerConnectionState->GameServerNotify = NewObject<UGameServerNotify>();
		GameServerConnectionState->GameServerNotify->Initialize(URL);
		GameServerConnectionState->GameServerNotify->InitNetDriver(GameServerConnectionState->NetDriver);
		GameServerConnectionState->GameServerNotify->SetFirstPlayer(DummyPlayer);
		GameServerConnectionState->GameServerNotify->SetProxyNetDriver(ProxyNetDriver);
		GameServerConnectionState->GameServerNotify->SetFlags(UE::Net::EJoinFlags::NoPawn);		
		GameServerConnection = GameServerConnectionState->NetDriver->ServerConnection;

		// UNetDriver::Notify will be reset in UPendingNetGame above so it's important that we override
		// it here again to point to the proxy.
		GameServerConnectionState->NetDriver->Notify = GameServerConnectionState->GameServerNotify;

		UE_LOGF(LogNetProxy, Log, "Connecting to game server (parent): %ls:%ls -> %ls (player=%ls client_handshake_id=%u)", 
			   *ProxyConnection->GetDriver()->GetName(), 
			   *ProxyConnection->GetName(),
			   *GameServerConnection->GetDriver()->GetName(),
			   *DummyPlayer->GetName(),
			   ParentClientHandshakeId);
	}

	{
		const uint32 ChildClientHandshakeId = ProxyNetDriver->GetNextClientHandshakeId();

		// Add a player to use on the game server.
		const FPlatformUserId GameServerClientId = FPlatformUserId::CreateFromInternalId(ProxyNetDriver->GetNextGameServerClientId());
		ULocalPlayer* NewPlayer = NewObject<ULocalPlayer>(GEngine, GEngine->LocalPlayerClass);
		NewPlayer->SetConnectionIdentifier(ChildClientHandshakeId);
		ProxyNetDriver->GetWorld()->GetGameInstance()->UGameInstance::AddLocalPlayer(NewPlayer, GameServerClientId);

		// The new player will use the same unique identifier as the incoming proxy connection so that it will be propogated up to 
		// the game servers through UNetConnection::PlayerId. This way each game server's incoming connection will have a PlayerId 
		// that corresponds to a client connected to the proxy.
		NewPlayer->SetCachedUniqueNetId(ProxyConnection->PlayerId);

		FGameServerSplitJoinRequest Request;
		Request.Player = NewPlayer;
		Request.Flags = Flags;
		Request.ClientHandshakeId = ChildClientHandshakeId;

		// The NMT_JoinSplit message can only be sent when the parent connection is open.
		if (GameServerConnectionState->NetDriver->ServerConnection->GetConnectionState() == USOCK_Open)
		{
			GameServerConnectionState->PendingSplitJoinRequests.Add(Request);
			ProxyNetDriver->FlushSplitJoinRequests(GameServerConnectionState);

			UE_LOGF(LogNetProxy, Log, "Connecting to game server (multiplexed): %ls:%ls -> %ls (player=%ls client_handshake_id=%u", 
				   *ProxyConnection->GetDriver()->GetName(), 
				   *ProxyConnection->GetName(),
				   *GameServerConnection->GetDriver()->GetName(),
				   *NewPlayer->GetName(),
				   Request.ClientHandshakeId);
		}
		else
		{
			GameServerConnectionState->PendingSplitJoinRequests.Add(Request);
	
			UE_LOGF(LogNetProxy, Log, "Connecting to game server (queued multiplexed): %ls:%ls -> %ls (player=%ls client_handshake_id=%u)", 
				   *ProxyConnection->GetDriver()->GetName(),
				   *ProxyConnection->GetName(),
				   *GameServerConnection->GetDriver()->GetName(),
				   *NewPlayer->GetName(),
				   Request.ClientHandshakeId);
		}
		
		GameServerConnectionState->Players.Add(NewPlayer);
		
		// Associate this proxy connection with the parent game server connection when beginning the handshake because
		// the child connection hasn't been created yet. Once the handshake is complete, it's expected that this entry will
		// be updated with the new child connection.
		FMultiServerProxyInternalConnectionRoute Route;
		Route.ClientHandshakeId = ChildClientHandshakeId;
		Route.ProxyConnection = ProxyConnection;
		Route.ParentGameServerConnection = GameServerConnection;
		Route.GameServerConnection = nullptr;
		Route.Player = NewPlayer;
		Route.PlayerController = nullptr;
		Route.LastUpdateViewTargetSec = 0.0;
		Route.LastViewTargetPos = FVector(0, 0, 0);

		if (!EnumHasAllFlags(Flags, UE::Net::EJoinFlags::NoPawn))
		{
			Route.State = EProxyConnectionState::ConnectingPrimary;
		}
		else
		{
			Route.State = EProxyConnectionState::Connecting;
		}
	
		ProxyNetDriver->InternalRoutes.Add(ChildClientHandshakeId, Route);
	}
}

void UProxyBackendNetConnection::HandleClientPlayer(APlayerController* NewPlayerController, UNetConnection* GameServerConnection)
{
	Super::HandleClientPlayer(NewPlayerController, GameServerConnection);

	NewPlayerController->NetConnection = GameServerConnection;

	// Send any queued join requests for pending multiplexed connections.
	UProxyBackendNetDriver* BackendNetDriver = Cast<UProxyBackendNetDriver>(Driver);
	if (!ensure(BackendNetDriver))
	{
		return;
	}

	UProxyNetDriver* ProxyNetDriver = Cast<UProxyNetDriver>(BackendNetDriver->GetProxyNetDriver());
	if (ensure(BackendNetDriver) && ensure(ProxyNetDriver))
	{
		for (int32 GameServerIndex = 0; GameServerIndex < ProxyNetDriver->GetGameServerConnectionCount(); GameServerIndex++)
		{
			if (FGameServerConnectionState* GameServerConnectionState = ProxyNetDriver->GetGameServerConnection(GameServerIndex))
			{
				if (GameServerConnectionState->NetDriver->ServerConnection == GameServerConnection)
				{
					ProxyNetDriver->FlushSplitJoinRequests(GameServerConnectionState);
					break;
				}
			}
		}
	}

	// The parent connection is not associated with a proxy client connection so it doesn't need to replicate state down to the
	// proxy. Only child connections are associated with clients. For this reason, replication to this connection on the backend
	// game server can be disabled.
	if (!UE::MultiServerProxy::Private::bEnableParentConnectionReplication)
	{
		bool bEnableReplication = false;
		UFunction* Function = ANoPawnPlayerController::StaticClass()->FindFunctionByName(TEXT("ServerEnableReplicationToConnection"));
		GameServerConnection->Driver->ProcessRemoteFunction(Cast<ANoPawnPlayerController>(NewPlayerController), Function, &bEnableReplication, nullptr, nullptr);
	}
}

void UProxyBackendChildNetConnection::RemoveLocalPlayer(UGameInstance* GameInstance, ULocalPlayer* LocalPlayer)
{
	GameInstance->UGameInstance::RemoveLocalPlayer(LocalPlayer);
}

void UProxyBackendChildNetConnection::HandleClientPlayer(APlayerController* NewPlayerController, UNetConnection* GameServerConnection)
{
	// This function is called when a PlayerController is replicated to the proxy from a game server and represents the finalization
	// of a connection to a primary or non-primary game server.

	ensure(GameServerConnection != this);

	UProxyBackendNetDriver* BackendNetDriver = Cast<UProxyBackendNetDriver>(Driver);
	if (ensure(BackendNetDriver))
	{
		BackendNetDriver->GetProxyNetDriver()->GameServerAssignPlayerController(this, GameServerConnection, NewPlayerController);
	}
}

void UProxyBackendNetDriver::SetProxyNetDriver(TObjectPtr<UProxyNetDriver> InProxyNetDriver)
{
	ProxyNetDriver = InProxyNetDriver;
}

TObjectPtr<UProxyNetDriver> UProxyBackendNetDriver::GetProxyNetDriver()
{
	return ProxyNetDriver;
}

bool UProxyBackendNetDriver::InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error)
{
	// Called before InitBase() because the values can be used during driver initialization.
	EnableExecuteRPCFunctions(false);
	SetProxyType(UE::Net::EProxyType::Backend);

	bool bSuccess = Super::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error);

	if (bSuccess)
	{
		NetConnectionClass = UProxyBackendNetConnection::StaticClass();
		ChildNetConnectionClass = UProxyBackendChildNetConnection::StaticClass();

		if (GetReplicationSystem())
		{
			if (UObjectReplicationBridge* Bridge = GetReplicationSystem()->GetReplicationBridge())
			{
				using namespace UE::Net::Private;

				OnIrisRootObjectPostInitHandle = FObjectReplicationBridgeDelegates::GetRootObjectPostInitDelegate(Bridge).AddUObject(this, &UProxyBackendNetDriver::OnIrisRootObjectPostInit);
				OnIrisRootObjectDetachedHandle = FObjectReplicationBridgeDelegates::GetRootObjectDetachedDelegate(Bridge).AddUObject(this, &UProxyBackendNetDriver::OnIrisRootObjectDetached);
				OnIrisSubObjectPostInitHandle = FObjectReplicationBridgeDelegates::GetSubObjectPostInitDelegate(Bridge).AddUObject(this, &UProxyBackendNetDriver::OnIrisSubObjectPostInit);
				OnIrisSubObjectDetachedHandle = FObjectReplicationBridgeDelegates::GetSubObjectDetachedDelegate(Bridge).AddUObject(this, &UProxyBackendNetDriver::OnIrisSubObjectDetached);
			}
		}
	}

	SetReplicateTransactionally(false);

	return bSuccess;
}

void UProxyBackendNetDriver::Shutdown()
{
	if (GetReplicationSystem())
	{
		if (UObjectReplicationBridge* Bridge = GetReplicationSystem()->GetReplicationBridge())
		{
			UE::Net::Private::FObjectReplicationBridgeDelegates::GetRootObjectPostInitDelegate(Bridge).Remove(OnIrisRootObjectPostInitHandle);
			UE::Net::Private::FObjectReplicationBridgeDelegates::GetRootObjectDetachedDelegate(Bridge).Remove(OnIrisRootObjectDetachedHandle);
			UE::Net::Private::FObjectReplicationBridgeDelegates::GetSubObjectPostInitDelegate(Bridge).Remove(OnIrisSubObjectPostInitHandle);
			UE::Net::Private::FObjectReplicationBridgeDelegates::GetSubObjectDetachedDelegate(Bridge).Remove(OnIrisSubObjectDetachedHandle);
		}
	}

	Super::Shutdown();
}

void UProxyBackendNetDriver::ForwardRemoteFunction(UObject* RootObject, UObject* SubObject, UFunction* Function, void* Parms)
{
	check(ProxyNetDriver);

	// This function is called when the proxy receives an RPC from a game server and will only forward the function on
	// to the frontend net driver if it is owned by a player that considers that game server the primary game server.

	ProxyNetDriver->GetMetrics()->IncrementInt(UE::MultiServerProxy::Metrics::NumForwardClientRPC, 1);

	AActor* OwningActor = Cast<AActor>(RootObject);
	if (!ensure(OwningActor != nullptr))
	{
		UE_LOGF(LogNetProxy, Verbose, "RPC: Ignoring %ls on %ls from game server %ls because it has no owner.", *Function->GetName(), *RootObject->GetName(), *GetName());
		return;
	}

	// Drop RPCs that don't come from a player controller on a primary game server.
	if (APlayerController* PlayerController = Cast<APlayerController>(OwningActor))
	{
		// This can happen when a proxy receives a parent connection RPC from the game server. These should be ignored because no proxy
		// client is associated with the parent connection and therefore it cannot forward the RPC.
		if (PlayerController->GetNetConnection() && Cast<UProxyBackendChildNetConnection>(PlayerController->GetNetConnection()) == nullptr)
		{
			UE_LOGF(LogNetProxy, Verbose, "RPC: Ignoring %ls on %ls from game server %ls because it's coming from a parent connection %ls.", *Function->GetName(), *RootObject->GetName(), *GetName(), *PlayerController->GetNetConnection()->GetName());
			return;
		}

		// This can happen when a connection to a backend game server was closed on the proxy but the NMT_CloseChildConnection message hasn't
		// be processed by the server yet and is still sending actor RPCs.
		if (PlayerController->NetConnection == nullptr)
		{
			UE_LOGF(LogNetProxy, Verbose, "RPC: Ignoring %ls on %ls from game server %ls because player controller %ls has a null network connection.", *Function->GetName(), *RootObject->GetName(), *GetName(), *GetNameSafe(PlayerController));
			return;
		}

		const FMultiServerProxyInternalConnectionRoute* Route = ProxyNetDriver->GetRouteWithPlayerController(PlayerController);
		if (!ensure(Route))
		{
			return;
		}

		if (Route->State != EProxyConnectionState::ConnectedPrimary)
		{
			UE_LOGF(LogNetProxy, Verbose, "RPC: Ignoring %ls on %ls from game server %ls because it's owning player controller %ls cannot map to a primary route.", *Function->GetName(), *RootObject->GetName(), *GetName(), *PlayerController->GetName());
			return;
		}
	}

	ULocalPlayer* OwningPlayer = Cast<ULocalPlayer>(OwningActor->GetNetOwningPlayerAnyRole());
	if (OwningPlayer == nullptr)
	{
		UE_LOGF(LogNetProxy, Verbose, "RPC: Ignoring %ls on %ls from game server %ls because it's owning actor %ls doesn't have an owning player.", *Function->GetName(), *RootObject->GetName(), *GetName(), *OwningActor->GetName());
		return;
	}

	UE_LOGF(LogNetProxy, VeryVerbose, "RPC: Pre-forwarding %ls on %ls from game server %ls to proxy connection.", *Function->GetName(), *OwningActor->GetName(), *GetName());

	if (UReplicationSystem* ProxyReplicationSystem = ProxyNetDriver->GetReplicationSystem())
	{
		ProxyNetDriver->RedirectRPCToProxyConnection(OwningActor, Function, [ProxyReplicationSystem, OwningActor, SubObject, Function, Parms](UNetConnection* ProxyConnection) {
			ProxyReplicationSystem->SendRPC(ProxyConnection->GetConnectionHandle().GetParentConnectionId(), OwningActor, SubObject, Function, Parms);
		});
	}
	else
	{
		ProxyNetDriver->RedirectRPCToProxyConnection(OwningActor, Function, [this, OwningActor, SubObject, Function, Parms](UNetConnection* ProxyConnection) {
			ProxyNetDriver->InternalProcessRemoteFunction(OwningActor, SubObject, ProxyConnection, Function, Parms, nullptr, nullptr, true);
		});
	}

	const FSoftObjectPath FunctionPath(Function->GetOwnerClass()->GetClassPathName(), Function->GetName());
	if (ProxyNetDriver->AllowListedProxyFunctions.Contains(FunctionPath))
	{
		UE_LOGF(LogNetProxy, Verbose, "RPC: Executing allow-listed %ls on %ls on proxy (game server to client) from %ls", *Function->GetName(), *GetNameSafe(RootObject), *GetName());
		UE::MultiServerProxy::Private::FScopedPlayerControllerPlayerSwap PlayerSwap(Cast<APlayerController>(RootObject), ProxyNetDriver);
		RootObject->ProcessEvent(Function, Parms);
	}
}

bool UProxyBackendNetDriver::ShouldSkipRepNotifies() const
{
	return true;
}

bool UProxyBackendNetDriver::ShouldClientDestroyActor(AActor* Actor, EChannelCloseReason CloseReason) const
{
	// If an actor is destroyed remotely on a game server because it's being migrated to another server then it shouldn't be
	// destroyed on the proxy when removed from the replication system of the originating server. When the actor arrives on
	// the destination server it will be added to that server's replication system and the actor will be re-used on the proxy
	// since it's still referenced in the shared backend NetGUID cache.
	return (CloseReason != EChannelCloseReason::Migrated);
}

bool UProxyBackendNetDriver::ShouldUpdateStats() const
{
	return true;
}

bool UProxyBackendNetDriver::ShouldRegisterMetricsDatabaseListeners() const
{
	// Metrics database listeners are not used by UProxyBackendNetDriver because they are aggregated in UProxyNetDriver 
	// and reported to the metrics database listeners.

	return false;
}

void UProxyBackendNetDriver::OnIrisRootObjectPostInit(UE::Net::FNetRefHandle RootHandle, UObject* RootObject, UE::Net::FNetObjectFactoryId FactoryId)
{
	if (!GetReplicationSystem())
	{
		return;
	}

	if (!ProxyNetDriver || !ProxyNetDriver->GetReplicationSystem())
	{
		return;
	}
	
	UObjectReplicationBridge* BackendBridge = GetReplicationSystem()->GetReplicationBridgeAs<UObjectReplicationBridge>();
	if (!BackendBridge)
	{
		return;
	}

	UObjectReplicationBridge* FrontendBridge = ProxyNetDriver->GetReplicationSystem()->GetReplicationBridgeAs<UObjectReplicationBridge>();
	if (!FrontendBridge)
	{
		return;
	}

	UE_LOGF(LogNetProxy, Verbose, "[%ls] Attaching root object %ls (%ls) to %ls", *GetNameSafe(this), *GetNameSafe(RootObject), *RootHandle.ToString(), *GetNameSafe(ProxyNetDriver));
	FrontendBridge->StartReplicatingRootObject(RootObject, FactoryId);

	if (AActor* Actor = Cast<AActor>(RootObject))
	{
		if (UEngineReplicationBridge* EngineFrontendBridge = Cast<UEngineReplicationBridge>(FrontendBridge))
		{
			EngineFrontendBridge->AddRootObjectToContainerGroup(Actor, Actor->GetLevel());
		}
	}
}

void UProxyBackendNetDriver::OnIrisRootObjectDetached(UE::Net::FNetRefHandle RootHandle, UObject* RootObject, UE::Net::EDetachReason Reason)
{
	if (!GetReplicationSystem())
	{
		return;
	}

	if (!ProxyNetDriver || !ProxyNetDriver->GetReplicationSystem())
	{
		return;
	}
	
	UObjectReplicationBridge* BackendBridge = GetReplicationSystem()->GetReplicationBridgeAs<UObjectReplicationBridge>();
	if (!BackendBridge)
	{
		return;
	}

	UObjectReplicationBridge* FrontendBridge = ProxyNetDriver->GetReplicationSystem()->GetReplicationBridgeAs<UObjectReplicationBridge>();
	if (!FrontendBridge)
	{
		return;
	}

	UE_LOGF(LogNetProxy, Verbose, "[%ls] Detaching root object %ls (%ls) from %ls", *GetNameSafe(this), *GetNameSafe(RootObject), *RootHandle.ToString(), *GetNameSafe(ProxyNetDriver));
	const EEndReplicationFlags Flags = (Reason == UE::Net::EDetachReason::TornOff) ? EEndReplicationFlags::TearOff : EEndReplicationFlags::Destroy;
	FrontendBridge->StopReplicatingNetObject(RootObject, Flags);
}

void UProxyBackendNetDriver::OnIrisSubObjectPostInit(UE::Net::FNetRefHandle SubObjectHandle, UObject* SubObject, UE::Net::FNetRefHandle RootObjectHandle, UObject* RootObject, UE::Net::FNetObjectFactoryId FactoryId)
{
	if (!GetReplicationSystem())
	{
		return;
	}

	if (!ProxyNetDriver || !ProxyNetDriver->GetReplicationSystem())
	{
		return;
	}

	UObjectReplicationBridge* BackendBridge = GetReplicationSystem()->GetReplicationBridgeAs<UObjectReplicationBridge>();
	if (!BackendBridge)
	{
		return;
	}

	UObjectReplicationBridge* FrontendBridge = ProxyNetDriver->GetReplicationSystem()->GetReplicationBridgeAs<UObjectReplicationBridge>();
	if (!FrontendBridge)
	{
		return;
	}
	
	UE::Net::FNetRefHandle FrontendRootObjectHandle = FrontendBridge->GetReplicatedRefHandle(RootObject);
	if (FrontendRootObjectHandle.IsValid())
	{
		UE_LOGF(LogNetProxy, Verbose, "[%ls] Attaching sub-object %ls (%ls) to %ls (%ls) on %ls", 
			   *GetNameSafe(this), 
			   *GetNameSafe(SubObject), *SubObjectHandle.ToString(), 
			   *GetNameSafe(RootObject), *RootObjectHandle.ToString(), 
			   *GetNameSafe(ProxyNetDriver));

		UObjectReplicationBridge::FSubObjectReplicationParams Params;
		Params.RootObjectHandle = FrontendRootObjectHandle;
		FrontendBridge->StartReplicatingSubObject(SubObject, Params, FactoryId);
	}
}

void UProxyBackendNetDriver::OnIrisSubObjectDetached(UE::Net::FNetRefHandle SubObjectHandle, UObject* SubObject, UE::Net::FNetRefHandle RootObjectHandle, UObject* RootObject, UE::Net::EDetachReason Reason)
{
	if (!GetReplicationSystem())
	{
		return;
	}

	if (!ProxyNetDriver || !ProxyNetDriver->GetReplicationSystem())
	{
		return;
	}

	UObjectReplicationBridge* BackendBridge = GetReplicationSystem()->GetReplicationBridgeAs<UObjectReplicationBridge>();
	if (!BackendBridge)
	{
		return;
	}

	UObjectReplicationBridge* FrontendBridge = ProxyNetDriver->GetReplicationSystem()->GetReplicationBridgeAs<UObjectReplicationBridge>();
	if (!FrontendBridge)
	{
		return;
	}

	UE_LOGF(LogNetProxy, Verbose, "[%ls] Detaching sub-object %ls (%ls) from %ls (%ls) on %ls", 
		   *GetNameSafe(this), 
		   *GetNameSafe(SubObject), *SubObjectHandle.ToString(),
		   *GetNameSafe(RootObject), *RootObjectHandle.ToString(),
		   *GetNameSafe(ProxyNetDriver));

	const EEndReplicationFlags Flags = (Reason == UE::Net::EDetachReason::TornOff) ? EEndReplicationFlags::TearOff : EEndReplicationFlags::Destroy;
	FrontendBridge->StopReplicatingNetObject(SubObject, Flags);
}

void UProxyNetDriver::RegisterGameServer(const FURL& GameServerURL)
{
	UE_LOGF(LogNetProxy, Log, "Registering Game Server for URL: %ls", *GameServerURL.ToString());
	GameServerConnections.Emplace(GameServerURL);
}

void UProxyNetDriver::SetCyclePrimaryGameServer(bool bEnable)
{
	bCyclePrimaryGameServer = bEnable;
}

bool UProxyNetDriver::IsConnectedToAllGameServers() const
{
	for (const FGameServerConnectionState& GameServerConnectionState : GameServerConnections)
	{
		if (GameServerConnectionState.NetDriver == nullptr ||
			GameServerConnectionState.NetDriver->ServerConnection == nullptr ||
			GameServerConnectionState.NetDriver->ServerConnection->GetConnectionState() != EConnectionState::USOCK_Open)
		{
			return false;
		}
	}

	return true;
}

int32 UProxyNetDriver::GetGameServerConnectionCount() const
{
	return GameServerConnections.Num();
}

FGameServerConnectionState* UProxyNetDriver::GetGameServerConnection(int32 Index)
{
	if (ensure(Index < GameServerConnections.Num()))
	{
		return &GameServerConnections[Index];
	}

	return nullptr;
}

int32 UProxyNetDriver::GetGameServerChildConnectionCount() const
{
	int32 ChildConnections = 0;

	for (const FGameServerConnectionState& GameServerConnection : GameServerConnections)
	{
		if (GameServerConnection.NetDriver && GameServerConnection.NetDriver->ServerConnection)
		{
			ChildConnections += GameServerConnection.NetDriver->ServerConnection->Children.Num();
		}
	}

	return ChildConnections;
}

// Given a comma-separated list of strings (IpAddress1:StartPort-EndPort,IpAddress2:StartPort-EndPort)
// Return the list of actual servers addresses: { IpAddress1:StartPort, ..., IpAddress1:EndPort, IpAddress2:StartPort, ... }
// This allows us to use short-hand when connecting to multiple servers.
TArray<FString> ParseServerAddressList(const FString& InServerAddressesStr)
{
	TArray<FString> InAddresses;
	InServerAddressesStr.ParseIntoArray(InAddresses, TEXT(","), true);

	TArray<FString> OutAddresses;
	for (const FString& InAddress : InAddresses)
	{
		// See if the address comes with a port (e.g. IpAddress:PortRange)
		int32 PortSeparatorIdx = INDEX_NONE;
		if (InAddress.FindLastChar(TEXT(':'), PortSeparatorIdx))
		{
			FString AddressPart = InAddress.Left(PortSeparatorIdx);
			FString PortPart = InAddress.Mid(PortSeparatorIdx + 1);

			// See if the PortRange is actually a port range (:StartPort-EndPort) or just (:Port)
			int32 RangeSeparatorIdx = INDEX_NONE;
			if (PortPart.FindLastChar(TEXT('-'), RangeSeparatorIdx))
			{
				FString StartPortView = PortPart.Left(RangeSeparatorIdx);
				FString EndPortView = PortPart.Mid(RangeSeparatorIdx + 1);
				int32 StartPort = FCString::Atoi(*StartPortView);
				int32 EndPort = FCString::Atoi(*EndPortView);

				if (StartPort > 0 && EndPort >= StartPort)
				{
					for (int32 Port = StartPort; Port <= EndPort; ++Port)
					{
						OutAddresses.Add(FString::Printf(TEXT("%s:%d"), *AddressPart, Port));
					}
				}
			}
			else
			{
				// Just put the address in as typed (we don't need to verify the port)
				OutAddresses.Emplace(InAddress);
			}
		}
		else
		{
			// Just put the address in as typed (no port specified, let the system determine what to do)
			OutAddresses.Emplace(InAddress);
		}
	}

	return OutAddresses;
}

bool UProxyNetDriver::InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error)
{
	check(!bInitAsClient);

	UE_LOGF(LogNetProxy, Log, "Initializing ProxyNetDriver with URL %ls", *URL.ToString());

	ProxyListenerNotify = NewObject<UProxyListenerNotify>(GEngine, UProxyListenerNotify::StaticClass());
	ProxyListenerNotify->SetProxyNetDriver(this);
	
	// Called before InitBase() because the values can be used during driver initialization.
	DisableActorLogicAndGameCode();
	EnableRoleSwappingOfAllActors(true);
	EnableReplicationOfActorsWithAnyRole(true);
	SetProxyType(UE::Net::EProxyType::Frontend);

	const bool bSuccess = Super::InitBase(bInitAsClient, ProxyListenerNotify, URL, bReuseAddressAndPort, Error);

	if (bSuccess)
	{
		NetConnectionClass = UProxyNetConnection::StaticClass();
		ProxyPreGarbageCollectHandle = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddUObject(this, &UProxyNetDriver::PreGarbageCollect);
		ProxyPostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UProxyNetDriver::PostGarbageCollect);
	}

	FString GameServerAddresses;
	if (FParse::Value(FCommandLine::Get(), TEXT("-ProxyGameServers="), GameServerAddresses, false))
	{
		TArray<FString> AllGameServerAddresses = ParseServerAddressList(GameServerAddresses);

		for (const FString& Address : AllGameServerAddresses)
		{
			FURL GameServerURL{ nullptr, *Address, ETravelType::TRAVEL_Absolute };
			if (ensureMsgf(GameServerURL.Valid, TEXT("Attempting to register an invalid ProxyGameServer URL. Input: %s. Parsed URL: %s"), *Address, *GameServerURL.ToString()))
			{
				RegisterGameServer(GameServerURL);
			}
		}
	}

	FString ClientPrimaryGameServer;
	if (FParse::Value(FCommandLine::Get(), TEXT("ProxyClientPrimaryGameServer="), ClientPrimaryGameServer))
	{
		if (ClientPrimaryGameServer.Equals(TEXT("random"), ESearchCase::IgnoreCase))
		{
			bRandomizePrimaryGameServerForNextClient = true;
		}
		else
		{
			PrimaryGameServerForNextClient = FCString::Atoi(ToCStr(ClientPrimaryGameServer));
		}
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("ProxyCyclePrimaryGameServer")))
	{
		bCyclePrimaryGameServer = true;
	}

	SetReplicateTransactionally(false);

	// These functions need to be executed on the proxy to update Iris filter groups used for streaming on the frontend replication system.
	AllowListedProxyFunctions.Add(FSoftObjectPath(APlayerController::StaticClass()->GetClassPathName(), GET_FUNCTION_NAME_CHECKED(APlayerController, ServerUpdateLevelVisibility).ToString()));
	AllowListedProxyFunctions.Add(FSoftObjectPath(APlayerController::StaticClass()->GetClassPathName(), GET_FUNCTION_NAME_CHECKED(APlayerController, ServerUpdateMultipleLevelsVisibility).ToString()));

	// RPCs from the client that must be forwarded to all game servers (not just the primary).

	// These functions need to be forwarded to game servers to know that the player can see a particular level.
	BroadcastToAllGameServersFunctions.Add(FSoftObjectPath(APlayerController::StaticClass()->GetClassPathName(), GET_FUNCTION_NAME_CHECKED(APlayerController, ServerUpdateLevelVisibility).ToString()));
	BroadcastToAllGameServersFunctions.Add(FSoftObjectPath(APlayerController::StaticClass()->GetClassPathName(), GET_FUNCTION_NAME_CHECKED(APlayerController, ServerUpdateMultipleLevelsVisibility).ToString()));

	GuidCache = TSharedPtr<FNetGUIDCache>(new FProxyNetGUIDCache(this));

	// All of the instances of UProxyBackendNetDriver will use this shared GUID cache to ensure that objects
	// with the same NetGUID use the same object pointers and don't create duplicate objects for the same NetGUID.
	SharedBackendNetGuidCache = TSharedPtr<FNetGUIDCache>(new FProxyBackendNetGUIDCache(this));

	return bSuccess;
}

void UProxyNetDriver::SetupNetworkMetrics()
{
	Super::SetupNetworkMetrics();
	
	GetMetrics()->CreateFloat(UE::MultiServerProxy::Metrics::FrameTimeMS, 0.0f);
	GetMetrics()->CreateFloat(UE::MultiServerProxy::Metrics::FPS, 0.0f);
	GetMetrics()->CreateInt(UE::MultiServerProxy::Metrics::NumForwardClientRPC, 0);
	GetMetrics()->CreateInt(UE::MultiServerProxy::Metrics::NumForwardBackendRPC, 0);
	GetMetrics()->CreateInt(UE::MultiServerProxy::Metrics::NumProxyRoutes, 0);
	GetMetrics()->CreateInt(UE::MultiServerProxy::Metrics::AggregatedBackendInRate, 0);
	GetMetrics()->CreateInt(UE::MultiServerProxy::Metrics::AggregatedBackendOutRate, 0);
	GetMetrics()->CreateInt(UE::MultiServerProxy::Metrics::NumReassignPlayerController, 0);
}

bool UProxyNetDriver::InitConnect(class FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error)
{
	checkf(0, TEXT("UProxyNetDriver is only intended to be used to receive connections and not establish outgoing connections."));
	return false;
}

void UProxyNetDriver::ForwardRemoteFunction(UObject* RootObject, UObject* SubObject, UFunction* Function, void* Parms)
{
	// This function is called when the proxy receives an RPC from a game client and will forward the function on
	// to the owning player's primary game server.

	GetMetrics()->IncrementInt(UE::MultiServerProxy::Metrics::NumForwardBackendRPC, 1);

	// If the owner is a PlayerController it will be for a proxy connection. There is no need to map it to the PlayerController
	// on the game server because UNetDriver::ProcessRemoteFunction() will automatically send the RPC to the server connection
	// and ignores the value of AActor::GetNetConnection().
	AActor* OwningActor = Cast<AActor>(RootObject);
	if (!ensure(OwningActor != nullptr))
	{
		UE_LOGF(LogNetProxy, Verbose, "RPC: Ignoring %ls on %ls from proxy connection because it doesn't have an owner.", *Function->GetName(), *RootObject->GetName());
		return;
	}

	ULocalPlayer* OwningPlayer = Cast<ULocalPlayer>(OwningActor->GetNetOwningPlayerAnyRole());
	if (OwningPlayer == nullptr)
	{
		UE_LOGF(LogNetProxy, Verbose, "RPC: Ignoring %ls on %ls from proxy connection because it owner %ls doesn't have an owning player", *Function->GetName(), *RootObject->GetName(), *OwningActor->GetName());
		return;
	}

	UE_LOGF(LogNetProxy, VeryVerbose, "RPC: Pre-forwarding %ls on %ls from proxy connection to game server.", *Function->GetName(), *OwningActor->GetName());

	// If the sub-object is not owned by the actor, attempt to find an component in that actor that matches the same type.
	// This logic assumes that an actor only has one component of a given type and will fail if that assumption is incorrect.
	bool bShouldForwardRPC = true;

	if (UActorComponent* SubObjectAsActorComponent = Cast<UActorComponent>(SubObject))
	{
		if (SubObjectAsActorComponent->GetOwner() != OwningActor)
		{
			int32 MatchingComponents = 0;
			UActorComponent* MappedActorComponent = SubObjectAsActorComponent;

			for (UActorComponent* ActorComponent : OwningActor->GetComponents())
			{
				if (ActorComponent->GetClass() == SubObjectAsActorComponent->GetClass())
				{
					MappedActorComponent = ActorComponent;
					MatchingComponents++;

					UE_LOGF(LogNetProxy, VeryVerbose, "RPC: Remapping sub-object %ls to sub-object %ls in actor %ls",
						   *SubObjectAsActorComponent->GetName(),
						   *MappedActorComponent ->GetName(),
						   *OwningActor->GetName());
				}
			}

			if (MappedActorComponent->GetOwner() != OwningActor)
			{
				bShouldForwardRPC = false;

				UE_LOGF(LogNetProxy, Verbose, "RPC: Unable map sub-object %ls to actor %ls",
					   *SubObjectAsActorComponent->GetName(),
					   *OwningActor->GetName());
			}

			// Detect an actor with two components of the same type.
			else if (MatchingComponents > 1)
			{
				bShouldForwardRPC = false;

				UE_LOGF(LogNetProxy, Verbose, "RPC: Found an actor %ls with more than one component %ls.", *OwningActor->GetName(), *SubObject->GetName());
			}
			else
			{
				SubObject = MappedActorComponent;
			}
		}
	}

	const FSoftObjectPath FunctionPath(Function->GetOwnerClass()->GetClassPathName(), Function->GetName());

	if (bShouldForwardRPC && AllowListedProxyFunctions.Contains(FunctionPath))
	{
		UE_LOGF(LogNetProxy, Verbose, "RPC: Executing allow-listed %ls on %ls on proxy (client to game server) for player %ls", *Function->GetName(), *GetNameSafe(OwningActor), *GetNameSafe(OwningPlayer));

		UE::MultiServerProxy::Private::FScopedPlayerControllerPlayerSwap PlayerSwap(Cast<APlayerController>(OwningActor), this);
		OwningActor->ProcessEvent(Function, Parms);
	}

	const bool bShouldBroadcastToAllGameServers = bShouldForwardRPC && BroadcastToAllGameServersFunctions.Contains(FunctionPath);
	if (bShouldBroadcastToAllGameServers)
	{
		BroadcastToAllClientConnectedGameServers(OwningPlayer, Function, Parms, SubObject);
	}
	else if (TObjectPtr<const APlayerController>* GamePlayerControllerPtr = PlayerToPrimaryGameServerPlayerController.Find(OwningPlayer))
	{
		// Forward all other RPCs to the primary game server only.
		UNetConnection* GameConnection = GamePlayerControllerPtr->Get()->NetConnection;
		if (GameConnection)
		{
			const bool bIsConnectionToGameServer = GameConnection->IsA(UProxyBackendNetConnection::StaticClass()) || GameConnection->IsA(UProxyBackendChildNetConnection::StaticClass());
			if (ensure(bIsConnectionToGameServer))
			{
				if (bShouldForwardRPC)
				{
					UE_LOGF(LogNetProxy, Verbose, "RPC: Forwarding %ls on %ls to game server connection %ls:%ls for player %ls", *GetNameSafe(Function), *GetNameSafe(OwningActor), *GameConnection->Driver->GetName(), *GetNameSafe(GameConnection), *GetNameSafe(OwningPlayer));
					GameConnection->Driver->ProcessRemoteFunction(OwningActor, Function, Parms, static_cast<FOutParmRec*>(nullptr), static_cast<FFrame*>(nullptr), SubObject);
				}
				else
				{
					UE_LOGF(LogNetProxy, Verbose, "RPC: Ignoring %ls on %ls (sub-object:%ls) to game server connection %ls.",
						   *Function->GetName(),
						   *OwningActor->GetName(),
						   *GetNameSafe(SubObject),
						   *GameConnection->GetName());
				}
			}
			else
			{
				UE_LOGF(LogNetProxy, Error, "RPC: Unable to forward %ls on %ls received from a proxy connection because it's going to be routed back to the proxy connection", *Function->GetName(), *RootObject->GetName());
			}
		}
	}
	else
	{
		UE_LOGF(LogNetProxy, Verbose, "RPC: Unable to forward %ls on %ls because player %ls isn't mapped to a primary game server.", *Function->GetName(), *RootObject->GetName(), *OwningPlayer->GetName());
	}
}

UNetConnection* UProxyNetDriver::GetProxyConnectionForPlayer(const ULocalPlayer* Player) const
{
	if (const TObjectPtr<UNetConnection>* ProxyConnectionPtr = PlayerToProxyConnection.Find(Player))
	{
		return ProxyConnectionPtr->Get();
	}
	return nullptr;
}

void UProxyNetDriver::BroadcastToAllClientConnectedGameServers(const ULocalPlayer* Player, const UFunction* Function, const void* Parms, const UObject* SubObject)
{
	TObjectPtr<UNetConnection>* ProxyConnectionPtr = PlayerToProxyConnection.Find(Player);
	if (!ProxyConnectionPtr || !*ProxyConnectionPtr)
	{
		UE_LOGF(LogNetProxy, Warning, "BroadcastToAllClientConnectedGameServers: No proxy connection found for player %ls", *GetNameSafe(Player));
		return;
	}

	UNetConnection* ProxyConnection = *ProxyConnectionPtr;
	for (TPair<uint32, FMultiServerProxyInternalConnectionRoute>& Pair : InternalRoutes)
	{
		if (Pair.Value.ProxyConnection != ProxyConnection)
		{
			continue;
		}

		const bool bIsConnected = (Pair.Value.State == EProxyConnectionState::Connected || Pair.Value.State == EProxyConnectionState::ConnectedPrimary);
		if (bIsConnected)
		{
			UNetConnection* GameServerConn = Pair.Value.GameServerConnection;
			APlayerController* PC = const_cast<APlayerController*>(Pair.Value.PlayerController.Get());

			UE_LOGF(LogNetProxy, Verbose, "RPC: Broadcasting %ls on %ls to game server connection %ls:%ls",
				*GetNameSafe(Function), *GetNameSafe(PC), *GameServerConn->Driver->GetName(), *GetNameSafe(GameServerConn));

			GameServerConn->Driver->ProcessRemoteFunction(PC, const_cast<UFunction*>(Function), const_cast<void*>(Parms), static_cast<FOutParmRec*>(nullptr), static_cast<FFrame*>(nullptr), const_cast<UObject*>(SubObject));
		}
		else
		{
			Pair.Value.PendingBroadcastRPCs.Emplace(Function, Parms, SubObject);

			UE_LOGF(LogNetProxy, Verbose, "RPC: Queued broadcast %ls for route [%u] (not yet connected, state=%d, pending=%d)",
				   *GetNameSafe(Function), Pair.Value.ClientHandshakeId, static_cast<uint8>(Pair.Value.State), Pair.Value.PendingBroadcastRPCs.Num());
		}
	}
}

bool UProxyNetDriver::ShouldReplicateFunction(AActor* Actor, UFunction* Function) const
{
	// If any game code in the proxy world attempts to send an RPC it should not be called.
	return false;
}

void UProxyNetDriver::AddNetworkActor(AActor* Actor)
{
	// Ideally the proxy shouldn't spawn any actors since it's just used as a cache to pass state
	// between game clients and game servers. For now though, actors that have the role ROLE_Authority
	// will have replication disabled and the role set to ROLE_None. This stops them replicating to
	// clients and disable any game actor code that only performs when the role is ROLE_Authority.
	//
	// It's important to note this function is called for all actors spawned on the client, both ones
	// loaded by the proxy and those replicated from the connected game servers. It's assumed that 
	// the actors replicated from the game servers will not have a role of ROLE_Authority and will
	// therefore be unaffacted by this code and replicate as normal.
	//
	// This is ignored for APlayerController because it's set to ROLE_Authority temporarily in order
	// to correctly set it bReplicates flag to true when establishing a connection to a backend server.
	if (!Actor->IsA(APlayerController::StaticClass()))
	{
		if (Actor->GetIsReplicated())
		{
			if (Actor->GetLocalRole() == ROLE_Authority)
			{
				Actor->SetReplicates(false);
				Actor->SetRole(ROLE_None);
			}
		}
	}

	Super::AddNetworkActor(Actor);
}

void UProxyNetDriver::RemoveNetworkActor(AActor* Actor)
{
	// Iris will normally stop replicating an actor when AActor::EndReplication() is called, but actor callbacks 
	// are disabled on the proxy so it needs to be done here.
	if (UReplicationSystem* DriverReplicationSystem = GetReplicationSystem())
	{
		if (UEngineReplicationBridge* Bridge = Cast<UEngineReplicationBridge>(DriverReplicationSystem->GetReplicationBridge()))
		{
			Bridge->StopReplicatingActor(Actor, FStopReplicatingActorParams(EEndPlayReason::Destroyed));
		}
	}

	Super::RemoveNetworkActor(Actor);

	// Remove this actor from the proxy netguid cache. On a normal dedicated server there's a one-to-one
	// mapping between a NetGUID and the corresponding object - i.e. each unique object has a unique 
	// netguid. On the proxy an object can be created/destroyed repeatedly as it's migrated between
	// servers but will still maintain the same netguid (derived from a remote id). For this reason,
	// it's important to remove an object from the proxy netguid cache when it's being removed from
	// the replication system to avoid any conflicts in the cache which trigger checks/ensures.
	GuidCache->RemoveActorNetGUIDs(Actor);
}

bool UProxyNetDriver::ShouldCallRemoteFunction(UObject* Object, UFunction* Function, const FReplicationFlags& RepFlags) const
{
	return !RepFlags.bIgnoreRPCs;
}

int32 UProxyNetDriver::ServerReplicateActors(float DeltaSeconds)
{
	return Super::ServerReplicateActors(DeltaSeconds);
}

bool UProxyNetDriver::CanDowngradeActorRole(UNetConnection* ProxyConnection, AActor* Actor) const
{
	if (ULocalPlayer* Player = Cast<ULocalPlayer>(Actor->GetNetOwningPlayerAnyRole()))
	{
		// If this autonomous actor is owned by a player that is bound to the same proxy connection
		// as the attached proxy player controller, don't downgrade from ROLE_AutonomousProxy to ROLE_SimulatedProxy.
		if (Actor->GetLocalRole() == ROLE_AutonomousProxy)
		{
			if (const UNetConnection* PlayerProxyConnection = const_cast<UProxyNetDriver*>(this)->GetProxyConnectionFromPrimaryPlayer(Player))
			{
				if (PlayerProxyConnection == ProxyConnection)
				{
					return false;
				}
			}

			return true;
		}
	}

	return false;
}

void UProxyNetDriver::Shutdown()
{
	Super::Shutdown();

	for (FGameServerConnectionState& ConnectionState : GameServerConnections)
	{
		if (ConnectionState.NetDriver)
		{
			GEngine->DestroyNamedNetDriver(ConnectionState.World, ConnectionState.NetDriverName);
			ConnectionState.GameServerNotify->NetDriver = nullptr;
		}
	}

	GameServerConnections.Reset();
}

void UProxyNetDriver::FinishDestroy()
{
	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(ProxyPostGarbageCollectHandle);
	Super::FinishDestroy();
}

void UProxyNetDriver::PreGarbageCollect()
{
	for (TPair<uint32, FMultiServerProxyInternalConnectionRoute>& ClientHandshakeIdToRoute : InternalRoutes)
	{
		FMultiServerProxyInternalConnectionRoute* Route = &ClientHandshakeIdToRoute.Value;

		for (auto It = Route->PendingBroadcastRPCs.CreateIterator(); It; ++It)
		{
			if (It->Function.ResolveObjectPtr() == nullptr)
			{
				UNetConnection* GameServerConn = Route->GameServerConnection;

				UE_LOGF(LogNetProxy, Warning, "Dropping garbage collected function on %ls to game server connection %ls:%ls",
					   *GetNameSafe(Route->PlayerController), *GameServerConn->Driver->GetName(), *GetNameSafe(GameServerConn));

				It.RemoveCurrent();
			}
		}
	}
}

void UProxyNetDriver::PostGarbageCollect()
{
	for (auto It = PlayerToProxyConnection.CreateIterator(); It; ++It)
	{
		if (!It.Key().ResolveObjectPtr())
		{
			It.RemoveCurrent();
		}
	}
}

void UProxyNetDriver::TickFlush(float DeltaSeconds)
{
	TArray<TWeakObjectPtr<APlayerController>> PlayerControllerNetConnectionsToRestore;

	const bool bIris = (GetReplicationSystem() != nullptr);
	if (bIris)
	{
		// Enumerating through all of the actors in the world is not efficient, but it's currently the most
		// reliable way to ensure that all actor's owners are correctly set on the proxy.
		// 
		// Ideally the call to SetProxyConnectionAsOwner() should be made when an actor is registered with the 
		// replication system, but there isn't currently a hook in the proxy driver to do that: AddNetworkActor() 
		// and NotifyActorChannelOpen() cannot be used because they are called before the actor is registered 
		// in the replication system.
		for (TActorIterator<AActor> It(World); It; ++It)
		{
			const AActor* Actor = *It;

			if (!Actor->GetIsReplicated() || !(Actor->IsActorBeginningPlay() || Actor->HasActorBegunPlay()))
			{
				continue;
			}

			// Assumptions:
			//	* APlayerController::GetNetOwner()		Always returns a pointer to itself (APlayerController)
			//	* APawn::GetNetOwner()					Always returns a pointer to itself (APawn)
			//	* AActor::GetNetOwner()					Always return a pointer to either APlayerController or APawn.
			const AActor* NetOwner = Actor->GetNetOwner();

			ensureMsgf(!Actor->IsA<APlayerController>() || NetOwner == Actor,
				TEXT("APlayerController::GetNetOwner() should return itself, but %s returned %s"),
				*Actor->GetName(), *GetNameSafe(NetOwner));

			ensureMsgf(!Actor->IsA<APawn>() || NetOwner == Actor, 
				TEXT("APawn::GetNetOwner() should return itself, but %s returned %s"),
				*Actor->GetName(), *GetNameSafe(NetOwner));

			ensureMsgf(NetOwner == nullptr || NetOwner->IsA<APlayerController>() || NetOwner->IsA<APawn>(),
				TEXT("AActor::GetNetOwner() should return APlayerController or APawn, but %s returned %s (type %s)"),
				*Actor->GetName(), *NetOwner->GetName(), *NetOwner->GetClass()->GetName());

			if (NetOwner == nullptr)
			{
				continue;
			}

			UNetConnection* ProxyConnection = nullptr;
			for (TPair<TObjectPtr<UNetConnection>, TObjectPtr<const APlayerController>>& Pair : ProxyConnectionToPrimaryGameServerPlayerController)
			{
				if (NetOwner == Pair.Value || NetOwner == Pair.Value->GetPawn())
				{
					ProxyConnection = Pair.Key;
					break;
				}
			}

			if (ProxyConnection)
			{
				SetProxyConnectionAsOwner(ProxyConnection, Actor);
			}
		}
	}
	else
	{
		// In the future: We can iterate over some kind of PlayerController->Connection map to make this more efficient.
		// Reassign these PC NetConnections to the front-end (Client) Connections before replication
		// Any code that checks the NetConnection for replication (bNetOwner) ownership will be correct.
		for (TActorIterator<APlayerController> It(World); It; ++It)
		{
			APlayerController* PC = *It;
			const FMultiServerProxyInternalConnectionRoute* Route = GetRouteWithPlayerController(PC);
			if (Route)
			{
				ensureMsgf(PC->NetConnection == Route->GameServerConnection, TEXT("%s's NetConnection should have been pointing to the back-end GameServerConnection"), *PC->GetName());
				PC->NetConnection = Route->ProxyConnection;
				PlayerControllerNetConnectionsToRestore.Add(PC);
			}
		}
	}

	// Update any metrics in the database that aren't tracked while ticking the frame.
	UpdateProxyNetworkMetrics();

	PrepareStateForRelevancy();

	// The network stats and network metrics database are updated and reported to listeners here.
	Super::TickFlush(DeltaSeconds);

	// Reset any metrics that are tracked while ticking the frame.
	ResetProxyNetworkMetrics();

	if (!bIris)
	{
		// Restore the connection to the backend Game Servers
		for (TWeakObjectPtr<APlayerController>& WeakPC : PlayerControllerNetConnectionsToRestore)
		{
			if (APlayerController* PC = WeakPC.Get())
			{
				const FMultiServerProxyInternalConnectionRoute* Route = GetRouteWithPlayerController(PC);
				if (ensureMsgf(Route, TEXT("%s lost its MultiServerProxy Route during TickFlush"), *PC->GetName()))
				{
					ensureMsgf(PC->NetConnection == Route->ProxyConnection, TEXT("%s's NetConnection was (re-)created or (re-)assigned before TickFlush completed."), *PC->GetName());
					PC->NetConnection = Route->GameServerConnection;
				}
			}
		}

	}
}

int32 UProxyNetDriver::GetNextGameServerClientId()
{
	return NextGameServerClientId++;
}

uint32 UProxyNetDriver::GetNextClientHandshakeId()
{
	return NextClientHandshakeId++;
}

void UProxyNetDriver::DisableActorLogicAndGameCode()
{
	// The proxy should only be replicating the exact state from the servers and passing it on
	// to clients and not tick actors or call any user actor callbacks.

	EnableExecuteRPCFunctions(false);
	EnablePreReplication(false);

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	GetWorld()->EnableActorTickAndUserCallbacks(false);
#endif
}

void UProxyNetDriver::GameServerAssignPlayerController(UNetConnection* ChildGameServerConnection, UNetConnection* NotUsedParentGameServerConnection, APlayerController* GameServerPlayerController)
{
	const uint32 ClientHandshakeId = GameServerPlayerController->GetClientHandshakeId();
	FMultiServerProxyInternalConnectionRoute* Route = InternalRoutes.Find(ClientHandshakeId);
	if (!ensure(Route))
	{
		return;
	}

	UE_LOGF(LogNetProxy, Log, "[%d] Received a new player controller %ls:%ls for connection %ls:%ls for player %ls.", 
		   GameServerPlayerController->GetClientHandshakeId(),
		   *GameServerPlayerController->GetName(), *GameServerPlayerController->GetClass()->GetName(),
		   *ChildGameServerConnection->GetDriver()->GetName(), *ChildGameServerConnection->GetName(),
		   *GetNameSafe(Route->Player));

	EProxyConnectionState State = Route->State;

	switch (State)
	{
		case EProxyConnectionState::Disconnected:
			break;
		case EProxyConnectionState::Connecting:
			ProcessConnectionState_Connecting(Route, ChildGameServerConnection, GameServerPlayerController); 
			break;
		case EProxyConnectionState::ConnectingPrimary:
			ProcessConnectionState_ConnectingPrimary(Route, ChildGameServerConnection, GameServerPlayerController);
			break;
		case EProxyConnectionState::Connected:
			ProcessConnectionState_Connected(Route, ChildGameServerConnection, GameServerPlayerController);
			break;
		case EProxyConnectionState::ConnectedPrimary:
			ProcessConnectionState_ConnectedPrimary(Route, ChildGameServerConnection, GameServerPlayerController);
			break;
		case EProxyConnectionState::PendingReassign:
			ProcessConnectionState_Reassign(Route, ChildGameServerConnection, GameServerPlayerController);
			break;
		case EProxyConnectionState::PendingClose:
			ProcessConnectionState_PendingClose(Route, ChildGameServerConnection, GameServerPlayerController);
			break;
	}
}

TSharedPtr<FNetGUIDCache> UProxyNetDriver::GetSharedBackendNetGuidCache()
{
	return SharedBackendNetGuidCache;
}

void UProxyNetDriver::HandleClosedProxyConnection(UNetConnection* ProxyConnection)
{	
	TArray<uint32> RoutesToRemove;

	for (TPair<uint32, FMultiServerProxyInternalConnectionRoute>& ClientHandShakeIdToRoute : InternalRoutes)
	{
		const uint32 ClientHandshakeId = ClientHandShakeIdToRoute.Key;
		const FMultiServerProxyInternalConnectionRoute* Route = &ClientHandShakeIdToRoute.Value;

		if (Route->ProxyConnection == ProxyConnection)
		{
			RoutesToRemove.Add(ClientHandshakeId);
		}
	}

	UE_LOGF(LogNetProxy, Log, "Closing %d routes associated with proxy connection %ls.", 
		   RoutesToRemove.Num(),
		   *UE::MultiServerProxy::Private::ConnectionToString(ProxyConnection));

	for (uint32 ClientHandshakeIdToRemove : RoutesToRemove)
	{
		const FMultiServerProxyInternalConnectionRoute* Route = InternalRoutes.Find(ClientHandshakeIdToRemove);
		if (!ensure(Route))
		{
			continue;
		}

		// If the connection to the backend game server hasn't completed connecting then defer destroying the connection
		// until the connection handshake is complete.
		const bool bStillConnecting  = (Route->State == EProxyConnectionState::Connecting || Route->State == EProxyConnectionState::ConnectingPrimary);
		if (!bStillConnecting)
		{
			PlayerToProxyConnection.Remove(Route->Player);
			FMultiServerProxyInternalConnectionRoute RemovedRoute;
			if (InternalRoutes.RemoveAndCopyValue(ClientHandshakeIdToRemove, RemovedRoute))
			{
				CloseAndCleanUpConnectionRoute(&RemovedRoute);
			}
		}
		else
		{
			UE_LOGF(LogNetProxy, Log, "[%u] Deferring closing game server connection %ls using player controller %ls and player id %ls.",
				   Route->ClientHandshakeId,
				   *UE::MultiServerProxy::Private::ConnectionToString(Route->GameServerConnection),
				   *UE::MultiServerProxy::Private::PlayerControllerToString(Route->PlayerController),
				   *GetNameSafe(Route->Player));

			Route->State = EProxyConnectionState::PendingClose;
		}
	}
	
	ProxyConnectionToPrimaryGameServerPlayerController.Remove(ProxyConnection);
	DeferredReassignment.Remove(ProxyConnection);
}


void UProxyNetDriver::CloseAndCleanUpConnectionRoute(const FMultiServerProxyInternalConnectionRoute* Route)
{
	UE_LOGF(LogNetProxy, Log, "[%u] Closing game server connection %ls using player controller %ls and player id %ls.",
		   Route->ClientHandshakeId,
		   *UE::MultiServerProxy::Private::ConnectionToString(Route->GameServerConnection),
		   *UE::MultiServerProxy::Private::PlayerControllerToString(Route->PlayerController),
		   *GetNameSafe(Route->Player));

	UChildConnection* GameServerConnectionAsChild = Cast<UChildConnection>(Route->GameServerConnection);
	if (ensure(GameServerConnectionAsChild))
	{
		GameServerConnectionAsChild->CloseAndRemoveChild(UE::Net::ECloseChildFlags::SendCloseMessage);
	}
}

void UProxyNetDriver::FinalizeGameServerConnection(FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController)
{
	ensure(Route->PlayerController == nullptr);
	ensure(Route->GameServerConnection == nullptr);
	ensure(Route->ClientHandshakeId == PlayerController->GetClientHandshakeId());
	ensure(Route->ProxyConnection);

	PlayerController->Player = Route->Player;
	PlayerController->NetConnection = GameServerConnection;
	PlayerController->SetRole(ROLE_AutonomousProxy);

	// Set the replication flag on the player controller to ensure that it can be added to the Iris replication system. Calling it from here assumes 
	// that AActor::OnActorChannelOpen(), which ultimately calls FinalizeGameServerConnection(), is called before UEngineReplicationBridge::StartReplicatingActor().
	//
	// This only has an effect on Iris because an actor that doesn't have the replication flag set 
	ENetRole PreviousRole = PlayerController->GetLocalRole();
	PlayerController->SetRole(ROLE_Authority);
	PlayerController->SetReplicates(true);
	PlayerController->SetRole(PreviousRole);

	GameServerConnection->SetConnectionState(EConnectionState::USOCK_Open);
	GameServerConnection->SetClientHandshakeId(Route->ClientHandshakeId);
	GameServerConnection->PlayerController = PlayerController;
	GameServerConnection->OwningActor = PlayerController;
	GameServerConnection->LastReceiveTime = GetElapsedTime();

	Route->GameServerConnection = GameServerConnection;
	Route->PlayerController = PlayerController;
}

void UProxyNetDriver::ReassignPlayerController(const FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController)
{
	ensure(Route->State == EProxyConnectionState::Connected || Route->State == EProxyConnectionState::ConnectedPrimary || Route->State == EProxyConnectionState::PendingReassign);
	ensure(Route->ClientHandshakeId == PlayerController->GetClientHandshakeId());
	ensure(Route->GameServerConnection->GetClientHandshakeId() == PlayerController->GetClientHandshakeId());
	ensure(Route->GameServerConnection == GameServerConnection);

	GetMetrics()->IncrementInt(UE::MultiServerProxy::Metrics::NumReassignPlayerController, 1);

	const APlayerController* PrevPlayerController = Route->PlayerController;

	PlayerController->NetConnection = GameServerConnection;
	PlayerController->Player = Route->Player;
	Route->PlayerController = PlayerController;
	
	UE_LOGF(LogNetProxy, Log, "[%u] Replacing player controller %ls with %ls on route with game server connection %ls and player id %ls",
		   Route->ClientHandshakeId,
		   *UE::MultiServerProxy::Private::PlayerControllerToString(PrevPlayerController),
		   *UE::MultiServerProxy::Private::PlayerControllerToString(Route->PlayerController),
		   *UE::MultiServerProxy::Private::ConnectionToString(Route->GameServerConnection),
		   *GetNameSafe(Route->Player));

	// The proxy listens for 2 events when a player controller migrates from one game server to another:
	//	* The player controller on the connection to game server A will change to an instance of ANoPawnPlayerController.
	//	* The player controller on the connection to game server B will change to the instance of the game's player controller.
	//
	// IMPORTANT: 
	//		The code below must not assume that these events will arrive in a specific order, or even arrive at all.

	if (PlayerController->IsA(ANoPawnPlayerController::StaticClass()))
	{
		ReceivedReassignedNoPawnPlayerController(Route);
	}
	else
	{
		ReceivedReassignedGamePlayerController(Route);
	}
}

void UProxyNetDriver::ProcessConnectionState_Connecting(FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController)
{
	ensure(Route->State == EProxyConnectionState::Connecting);

	FinalizeGameServerConnection(Route, GameServerConnection, PlayerController);

	PlayerToProxyConnection.Add(Route->Player, Route->ProxyConnection);
	Route->State = EProxyConnectionState::Connected;

	FlushPendingBroadcastRPCs(Route);

	UE_LOGF(LogNetProxy, Log, "[%d] Successfully connected proxy connection %ls to non-primary game server connection %ls using player controller %ls and player %ls.",
		   Route->ClientHandshakeId,
		   *UE::MultiServerProxy::Private::ConnectionToString(Route->ProxyConnection),
		   *UE::MultiServerProxy::Private::ConnectionToString(GameServerConnection),
		   *UE::MultiServerProxy::Private::PlayerControllerToString(PlayerController),
		   *Route->Player->GetName());
}

void UProxyNetDriver::ProcessConnectionState_ConnectingPrimary(FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController)
{
	ensure(Route->State == EProxyConnectionState::ConnectingPrimary);

	FinalizeGameServerConnection(Route, GameServerConnection, PlayerController);

	Route->ProxyConnection->SetConnectionState(EConnectionState::USOCK_Open);
	Route->ProxyConnection->PlayerController = PlayerController;
	Route->ProxyConnection->OwningActor = PlayerController;
	Route->ProxyConnection->LastReceiveTime = GetElapsedTime();

	PlayerToPrimaryGameServerPlayerController.Add(Route->Player, Route->PlayerController);
	PlayerToProxyConnection.Add(Route->Player, Route->ProxyConnection);
	ProxyConnectionToPrimaryGameServerPlayerController.Add(Route->ProxyConnection, Route->PlayerController);

	Route->State = EProxyConnectionState::ConnectedPrimary;

	FlushPendingBroadcastRPCs(Route);

	UE_LOGF(LogNetProxy, Log, "[%d] Successfully connected proxy connection %ls to primary game server connection %ls using player controller %ls and player %ls.",
		   Route->ClientHandshakeId,
		   *UE::MultiServerProxy::Private::ConnectionToString(Route->ProxyConnection),
		   *UE::MultiServerProxy::Private::ConnectionToString(GameServerConnection),
		   *UE::MultiServerProxy::Private::PlayerControllerToString(PlayerController),
		   *Route->Player->GetName());
}

void UProxyNetDriver::ProcessConnectionState_Connected(const FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController)
{
	ReassignPlayerController(Route, GameServerConnection, PlayerController);
}

void UProxyNetDriver::ProcessConnectionState_ConnectedPrimary(const FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController)
{
	ReassignPlayerController(Route, GameServerConnection, PlayerController);
}

void UProxyNetDriver::ProcessConnectionState_Reassign(const FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController)
{
	ReassignPlayerController(Route, GameServerConnection, PlayerController);
}

void UProxyNetDriver::ProcessConnectionState_PendingClose(FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController)
{
	// This state is expected to be called when a connection is closed before it's completed the connection handshake.

	UE_LOGF(LogNetProxy, Log, "[%u] Closing connection %ls after handshake is completed using player controller %ls and player id %ls.",
		   Route->ClientHandshakeId,
		   *UE::MultiServerProxy::Private::ConnectionToString(GameServerConnection),
		   *UE::MultiServerProxy::Private::PlayerControllerToString(PlayerController),
		   *GetNameSafe(Route->Player));

	// Perform any initialization of the route, connection and player controller that is required to cleanly close
	// the connection.
	Route->GameServerConnection = GameServerConnection;
	GameServerConnection->PlayerController = PlayerController;

	PlayerToProxyConnection.Remove(Route->Player);
	FMultiServerProxyInternalConnectionRoute RemovedRoute;
	if (InternalRoutes.RemoveAndCopyValue(Route->ClientHandshakeId, RemovedRoute))
	{
		CloseAndCleanUpConnectionRoute(&RemovedRoute);
	}
}

void UProxyNetDriver::UpdateProxyNetworkMetrics()
{
	extern ENGINE_API float GAverageFPS;
	extern ENGINE_API float GAverageMS;

	GetMetrics()->SetFloat(UE::MultiServerProxy::Metrics::FrameTimeMS, GAverageMS);
	GetMetrics()->SetFloat(UE::MultiServerProxy::Metrics::FPS, GAverageFPS);
	GetMetrics()->SetInt(UE::MultiServerProxy::Metrics::NumProxyRoutes, InternalRoutes.Num());

	AggregateBackendNetDriverMetrics();
}

void UProxyNetDriver::AggregateBackendNetDriverMetrics()
{
	// Add together any metrics from the backend net drivers and store them in a metric in the frontend
	// proxy net driver which will be reported to metrics database listeners.

	static const TMap<FName, FName> BackendMetricToFrontendMetricInt = {
		{	UE::Net::Metric::InRate, 	UE::MultiServerProxy::Metrics::AggregatedBackendInRate	},
		{	UE::Net::Metric::OutRate, 	UE::MultiServerProxy::Metrics::AggregatedBackendOutRate	}
	};

	for (const TPair<FName, FName>& Pair : BackendMetricToFrontendMetricInt)
	{
		GetMetrics()->SetInt(Pair.Value, 0);
	}
	
	for (const FGameServerConnectionState& Connection : GameServerConnections)
	{
		if (!Connection.NetDriver)
		{
			continue;
		}

		UNetworkMetricsDatabase* BackendMetrics = Connection.NetDriver->GetMetrics();
		if (!BackendMetrics)
		{
			continue;
		}

		for (const TPair<FName, FName>& Pair : BackendMetricToFrontendMetricInt)
		{
			const FName BackendMetricName = Pair.Key;
			const FName FrontendMetricName = Pair.Value;

			ensure(BackendMetrics->Contains(BackendMetricName));
			ensure(GetMetrics()->Contains(FrontendMetricName));

			int64 BackendMetricValue = BackendMetrics->GetInt(BackendMetricName);
			GetMetrics()->IncrementInt(FrontendMetricName, BackendMetricValue);
		}
	}
}

void UProxyNetDriver::ResetProxyNetworkMetrics()
{
	// Reset any metrics that are incremented while ticking the net driver and not just set from existing values.

	GetMetrics()->SetInt(UE::MultiServerProxy::Metrics::NumForwardClientRPC, 0);
	GetMetrics()->SetInt(UE::MultiServerProxy::Metrics::NumForwardBackendRPC, 0);
	GetMetrics()->SetInt(UE::MultiServerProxy::Metrics::NumReassignPlayerController, 0);
}

void UProxyNetDriver::ReceivedReassignedNoPawnPlayerController(const FMultiServerProxyInternalConnectionRoute* Route)
{
	FPlayerControllerReassignment& Reassignment = DeferredReassignment.FindOrAdd(Route->ProxyConnection);

	// If we've already received a no pawn player controller, assume that a new migration has begun and reset
	// the state of the migration data structure. This can happen if a previous player controller migration didn't complete.
	if (Reassignment.bReceivedNoPawnPlayerController)
	{
		Reassignment = FPlayerControllerReassignment();

		UE_LOGF(LogNetProxy, Warning, "[%u] Resetting existing deferred player controller migration on proxy connection %ls.", 
			   Route->ClientHandshakeId,
			   *Route->ProxyConnection->GetName());
	}

	Reassignment.PreviousPrimaryPlayer = Route->Player;
	Reassignment.PreviousClientHandshakeId = Route->ClientHandshakeId;
	Reassignment.bReceivedNoPawnPlayerController = true;

	if (Reassignment.bReceivedNoPawnPlayerController && Reassignment.bReceivedGamePlayerController)
	{
		FinalizePlayerControllerReassignment(Route->ProxyConnection, Reassignment);
		DeferredReassignment.Remove(Route->ProxyConnection);
	}
	else
	{
		Route->State = EProxyConnectionState::PendingReassign;
	}
}

void UProxyNetDriver::RedirectRPCToProxyConnection(AActor* Actor, UFunction* Function, TFunction<void(UNetConnection*)> RPCSendingFunction)
{
	if (ULocalPlayer* Player = Cast<ULocalPlayer>(Actor->GetNetOwningPlayerAnyRole()))
	{
		// RPCs from game servers will be routed to proxy connections. Only RPCs from a connections primary game server will
		// be routed to the game client; other RPCs will be ignored.
		if (UNetConnection* ProxyConnection = GetProxyConnectionFromPrimaryPlayer(Player))
		{
			UPlayer* Owner = Actor->GetNetOwningPlayerAnyRole();
			UE_LOGF(LogNetProxy, Verbose, "RPC: Forwarding %ls on %ls (owner:%ls) to proxy connection %ls:%ls for player %ls", *GetNameSafe(Function), *GetNameSafe(Actor), *GetNameSafe(Owner), *ProxyConnection->Driver->GetName(), *GetNameSafe(ProxyConnection), *GetNameSafe(Player));
			RPCSendingFunction(ProxyConnection);
		}
		else
		{
			UE_LOGF(LogNetProxy, Verbose, "RPC: Ignoring %ls on %ls because player %ls isn't a primary player.", *Function->GetName(), *Actor->GetName(), *Player->GetName());
		}
	}
	else
	{
		UE_LOGF(LogNetProxy, Verbose, "RPC: Ignoring %ls on %ls because there is no actor owning player.", *Function->GetName(), *Actor->GetName());
	}
}

void UProxyNetDriver::SetProxyConnectionAsOwner(UNetConnection* ProxyConnection, const AActor* Actor)
{
	if (UReplicationSystem* DriverReplicationSystem = GetReplicationSystem())
	{
		if (UObjectReplicationBridge* ObjectReplicationBridge = DriverReplicationSystem->GetReplicationBridgeAs<UObjectReplicationBridge>())
		{
			const uint32 ConnectionId = ProxyConnection->GetConnectionHandle().GetParentConnectionId();
			const UE::Net::FNetRefHandle RefHandle = ObjectReplicationBridge->GetReplicatedRefHandle(Actor);
			if (RefHandle.IsValid())
			{
				DriverReplicationSystem->SetReplicationEnabledForConnection(ProxyConnection->GetConnectionHandle().GetParentConnectionId(), true);
				DriverReplicationSystem->SetOwningNetConnection(RefHandle, ConnectionId);

				// On the proxy, GetRemoteRole() returns ROLE_Authority (proxy is client of game server). Use GetLocalRole() which mirrors the game server's RemoteRole.
				const bool bEnableAutonomousCondition = Actor->GetLocalRole() == ROLE_AutonomousProxy;
				DriverReplicationSystem->SetReplicationConditionConnectionFilter(RefHandle, UE::Net::EReplicationCondition::RoleAutonomous, ConnectionId, bEnableAutonomousCondition);

				// Force net update when ownership is first assigned to ensure Role property re-send
				if (bEnableAutonomousCondition)
				{
					DriverReplicationSystem->ForceNetUpdate(RefHandle);
				}
			}
		}
	}
}

void UProxyNetDriver::ReceivedReassignedGamePlayerController(const FMultiServerProxyInternalConnectionRoute* Route)
{
	FPlayerControllerReassignment& Reassignment = DeferredReassignment.FindOrAdd(Route->ProxyConnection);

	// If we've already received a new game pawn player controller, assume that a new migration has begun and reset
	// the state of the migration data structure. This can happen if a previous player controller migration didn't complete.
	if (Reassignment.bReceivedGamePlayerController)
	{
		Reassignment = FPlayerControllerReassignment();

		UE_LOGF(LogNetProxy, Warning, "[%u] Resetting existing deferred player controller reassignment on proxy connection %ls.", 
			   Route->ClientHandshakeId,
			   *Route->ProxyConnection->GetName());
	}

	Reassignment.PrimaryPlayer = Route->Player;
	Reassignment.ClientHandshakeId = Route->ClientHandshakeId;
	Reassignment.bReceivedGamePlayerController = true;

	PlayerToPrimaryGameServerPlayerController.Add(Reassignment.PrimaryPlayer, Route->PlayerController);
	ProxyConnectionToPrimaryGameServerPlayerController.Add(Route->ProxyConnection, Route->PlayerController);

	if (Reassignment.bReceivedNoPawnPlayerController && Reassignment.bReceivedGamePlayerController)
	{
		FinalizePlayerControllerReassignment(Route->ProxyConnection, Reassignment);
		DeferredReassignment.Remove(Route->ProxyConnection);
	}
	else
	{
		Route->State = EProxyConnectionState::PendingReassign;
	}
}

void UProxyNetDriver::FinalizePlayerControllerReassignment(UNetConnection* ProxyConnection, const FPlayerControllerReassignment& Reassignment)
{
	// Perform any logic that depends on the order that the proxy receives reassigned game and no pawn player controllers, or having received
	// both controllers.

	UE_LOGF(LogNetProxy, Log, "Finalizing reassignment of primary player from route %lld, with player %ls, to route %lld and player %ls.",
		   Reassignment.PreviousClientHandshakeId,
		   *Reassignment.PreviousPrimaryPlayer->GetName(),
		   Reassignment.ClientHandshakeId,
		   *Reassignment.PrimaryPlayer->GetName());
	
	if (Reassignment.PreviousPrimaryPlayer != Reassignment.PrimaryPlayer)
	{
		PlayerToPrimaryGameServerPlayerController.Remove(Reassignment.PreviousPrimaryPlayer);
	}

	const FMultiServerProxyInternalConnectionRoute* PreviousRoute = InternalRoutes.Find(Reassignment.PreviousClientHandshakeId);
	const FMultiServerProxyInternalConnectionRoute* Route = InternalRoutes.Find(Reassignment.ClientHandshakeId);

	if (ensure(PreviousRoute))
	{
		PreviousRoute->State = EProxyConnectionState::Connected;
	}

	if (ensure(Route))
	{
		Route->State = EProxyConnectionState::ConnectedPrimary;
	}
}

void UProxyNetDriver::FlushSplitJoinRequests(FGameServerConnectionState* GameServerConnectionState)
{
	EConnectionState ParentConnectionState = GameServerConnectionState->NetDriver->ServerConnection->GetConnectionState();
	if (!(ParentConnectionState == EConnectionState::USOCK_Open))
	{
		UE_LOGF(LogNetProxy, Error, "Flushing split join requests on %ls without the parent connection being opened.", 
			   *GameServerConnectionState->NetDriver->GetName());

		return;
	}

	UE_LOGF(LogNetProxy, Log, "Flushing %d split join connection requests for connection %ls:%ls.", 
		   GameServerConnectionState->PendingSplitJoinRequests.Num(), *GameServerConnectionState->NetDriver->GetName(), *GameServerConnectionState->NetDriver->ServerConnection->GetName());

	for (FGameServerSplitJoinRequest& Request : GameServerConnectionState->PendingSplitJoinRequests)
	{
		UE_LOGF(LogNetProxy, Log, "Sending queued connection (multiplexed) request to game server: %ls (player=%ls flags=%d client_handshake_id=%u)",
			   *GameServerConnectionState->NetDriver->GetName(),
			   *Request.Player->GetName(),
			   EnumToUnderlyingType(Request.Flags),
			   Request.ClientHandshakeId);

		TArray<FString> Options;
		Options.Add(FString::Printf(TEXT("HandshakeId=%u"), Request.ClientHandshakeId));
		Request.Player->SendSplitJoin(Options, GameServerConnectionState->NetDriver, Request.Flags);
	}

	GameServerConnectionState->PendingSplitJoinRequests.Reset();
}

void UProxyNetDriver::FlushPendingBroadcastRPCs(FMultiServerProxyInternalConnectionRoute* Route)
{
	if (Route->PendingBroadcastRPCs.Num() == 0)
	{
		return;
	}

	if (!Route->GameServerConnection || !Route->PlayerController)
	{
		return;
	}

	UNetConnection* GameServerConn = Route->GameServerConnection;
	APlayerController* PC = const_cast<APlayerController*>(Route->PlayerController.Get());

	UE_LOGF(LogNetProxy, Verbose, "Flushing %d pending broadcast RPCs for route [%u] on connection %ls:%ls.",
		   Route->PendingBroadcastRPCs.Num(),
		   Route->ClientHandshakeId,
		   *GameServerConn->Driver->GetName(),
		   *GetNameSafe(GameServerConn));

	for (UE::MultiServerProxy::Private::FPendingBroadcastRPC& PendingRPC : Route->PendingBroadcastRPCs)
	{
		if (UFunction* Function = const_cast<UFunction*>(PendingRPC.Function.ResolveObjectPtr()))
		{
			UE_LOGF(LogNetProxy, Verbose, "RPC: Broadcasting queued %ls on %ls to game server connection %ls:%ls",
				   *GetNameSafe(Function), *GetNameSafe(PC), *GameServerConn->Driver->GetName(), *GetNameSafe(GameServerConn));

			GameServerConn->Driver->ProcessRemoteFunction(PC, const_cast<UFunction*>(Function), PendingRPC.Parms.GetData(), static_cast<FOutParmRec*>(nullptr), static_cast<FFrame*>(nullptr), const_cast<UObject*>(PendingRPC.SubObject.ResolveObjectPtr()));
		}
		else
		{
			UE_LOGF(LogNetProxy, Verbose, "RPC: Dropping queued invalid function on %ls to game server connection %ls:%ls",
				   *GetNameSafe(PC), *GameServerConn->Driver->GetName(), *GetNameSafe(GameServerConn));
		}
	}

	Route->PendingBroadcastRPCs.Reset();
}

UNetConnection* UProxyNetDriver::GetProxyConnectionFromPrimaryPlayer(ULocalPlayer* Player)
{
	TObjectPtr<const APlayerController>* GameServerPlayerControllerPtr = PlayerToPrimaryGameServerPlayerController.Find(Player);
	if (!GameServerPlayerControllerPtr)
	{
		return nullptr;
	}

	const APlayerController* GameServerPlayerController = GameServerPlayerControllerPtr->Get();

	const FMultiServerProxyInternalConnectionRoute* Route = GetRouteWithPlayerController(GameServerPlayerController);
	if (!Route)
	{
		return nullptr;
	}

	ensure(Route->Player == Player);

	return Route->ProxyConnection;
}

const FMultiServerProxyInternalConnectionRoute* UProxyNetDriver::GetRouteWithPlayerController(const APlayerController* PlayerController)
{
	return InternalRoutes.Find(PlayerController->GetClientHandshakeId());
}

void UProxyNetDriver::PrepareStateForRelevancy()
{
	for (TPair<TObjectPtr<UNetConnection>, TObjectPtr<const APlayerController>>& ProxyConnectionToPlayerController : ProxyConnectionToPrimaryGameServerPlayerController)
	{
		const APlayerController* PrimaryPlayerController = ProxyConnectionToPlayerController.Value;

		ensure(!PrimaryPlayerController->IsA(ANoPawnPlayerController::StaticClass()));

		UpdateLocalViewTarget(PrimaryPlayerController);
	}

	for (TPair<uint32, FMultiServerProxyInternalConnectionRoute>& ClientHandshakeIdToRoute : InternalRoutes)
	{
		const FMultiServerProxyInternalConnectionRoute* Route = &ClientHandshakeIdToRoute.Value;

		const ANoPawnPlayerController* NoPawnPlayerController = Cast<ANoPawnPlayerController>(Route->PlayerController);
		if (!NoPawnPlayerController)
		{
			continue;
		}

		TObjectPtr<const APlayerController>* PrimaryPlayerControllerPtr = ProxyConnectionToPrimaryGameServerPlayerController.Find(Route->ProxyConnection);
		if (!PrimaryPlayerControllerPtr)
		{
			continue;
		}

		const APlayerController* PrimaryPlayerController = *PrimaryPlayerControllerPtr;

		FVector Location;
		FRotator Rotation;
		PrimaryPlayerController->GetPlayerViewPoint(Location, Rotation);

		SetRemoteViewTarget(NoPawnPlayerController, Location);
	}

	// Relevancy calcluations are made using the position returned by AActor::GetActorLocation(). Since the proxy is not simulating
	// actor movement the actor's location must be manually set to the location in AActor::ReplicatedMovement.
	for (TActorIterator<AActor> It(World); It; ++It)
	{
		AActor* Actor = *It;

		if (Actor->GetIsReplicated() && Actor->IsReplicatingMovement())
		{
			Actor->SetActorLocation(Actor->GetReplicatedMovement().Location);
		}
	}
}

void UProxyNetDriver::UpdateLocalViewTarget(const APlayerController* PrimaryPlayerController)
{
	if (APawn* Pawn = PrimaryPlayerController->GetPawn())
	{
		UE_LOGF(LogNetProxy, VeryVerbose, "Setting the view target of %ls to %ls and updating the camera manager.",
			*GetNameSafe(Pawn),
			*GetNameSafe(PrimaryPlayerController));

	
		const_cast<APlayerController*>(PrimaryPlayerController)->SetViewTarget(Pawn);
	}
	else
	{
		UE_LOGF(LogNetProxy, Verbose, "Couldn't find a pawn for player controller %ls.", *GetNameSafe(PrimaryPlayerController));
	}
}

void UProxyNetDriver::SetRemoteViewTarget(const ANoPawnPlayerController* PlayerController, FVector ViewTargetPos)
{
	const FMultiServerProxyInternalConnectionRoute* PlayerControllerRoute = GetRouteWithPlayerController(PlayerController);
	if (!ensure(PlayerControllerRoute))
	{
		return;
	}

	if (PlayerControllerRoute->LastViewTargetPos.Equals(ViewTargetPos))
	{
		return;
	}
	
	const double WorldTimeSec = World->GetTimeSeconds();
	const double NextUpdateTime = (PlayerControllerRoute->LastUpdateViewTargetSec + UE::MultiServerProxy::Private::NonPrimarySetViewTargetIntervalSec);
	if (NextUpdateTime >= WorldTimeSec)
	{
		return;
	}

	PlayerControllerRoute->LastUpdateViewTargetSec = WorldTimeSec;
	PlayerControllerRoute->LastViewTargetPos = ViewTargetPos;
	
	UNetConnection* GameServerConnection = PlayerControllerRoute->GameServerConnection;

	UE_LOGF(LogNetProxy, VeryVerbose, "Updating view target position for %ls to %ls on connection %ls.", 
		   *PlayerController->GetName(), 
		   *ViewTargetPos.ToString(),
		   *GetNameSafe(GameServerConnection));

	UFunction* Function = ANoPawnPlayerController::StaticClass()->FindFunctionByName(TEXT("ServerSetViewTargetPosition"));
	GameServerConnection->Driver->ProcessRemoteFunction(const_cast<ANoPawnPlayerController*>(PlayerController), Function, &ViewTargetPos, nullptr, nullptr);
}

void UProxyNetConnection::CleanUp()
{
	if (UE::MultiServerProxy::Private::bEnableDisconnectionSupport)
	{
		if (Driver)
		{
			UProxyNetDriver* ProxyNetDriver = Cast<UProxyNetDriver>(Driver);
	
			if (ensure(ProxyNetDriver))
			{
				ProxyNetDriver->HandleClosedProxyConnection(this);
			}
		}
	}

	Super::CleanUp();
}
