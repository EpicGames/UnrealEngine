// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiServerReplicationModule.h"
#include "Modules/ModuleManager.h"
#include "UnrealEngine.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "MultiServerReplicationTypes.h"

namespace UE::MultiServerProxy::Private
{
	bool bProxyEnabled = false;
	static FAutoConsoleVariableRef CVarProxyEnabled(
		TEXT("net.proxy.Enabled"),
		bProxyEnabled,
		TEXT("When true, configure this game server to behave as a multi-server proxy."));

	enum class ENetDriverOverrideState
	{
		NoOverride,              // No -NetDriverOverrides specified
		ProxyDriverSpecified,    // ProxyNetDriver explicitly specified
		ConflictDetected         // Different driver specified for GameNetDriver
	};

	// Checks the -NetDriverOverrides command line argument for conflicts with proxy configuration.
	// Supports the same formats as the engine's NetDriverOverrides parsing:
	//   -NetDriverOverrides=DriverClassName (targets GameNetDriver by default)
	//   -NetDriverOverrides="DefName,DriverClassName"
	//   -NetDriverOverrides="DefName,DriverClassName,FallbackClassName"
	//   -NetDriverOverrides="DriverClassName;DefName2,DriverClassName2" (multiple entries)
	//
	// Returns the state of the -NetDriverOverrides command line argument.
	static ENetDriverOverrideState CheckForNetDriverOverrideConflict()
	{
		FString NetDriverOverridesValue;

		if (!FParse::Value(FCommandLine::Get(), TEXT("-NetDriverOverrides="), NetDriverOverridesValue))
		{
			return ENetDriverOverrideState::NoOverride;
		}

		// Parse entries separated by semicolons (same format as engine code)
		TArray<FString> OverrideEntries;
		NetDriverOverridesValue.ParseIntoArray(OverrideEntries, TEXT(";"), false);

		const FString ProxyNetDriverClassName = TEXT("/Script/MultiServerReplication.ProxyNetDriver");

		for (const FString& Entry : OverrideEntries)
		{
			TArray<FString> EntryParams;
			Entry.ParseIntoArray(EntryParams, TEXT(","), false);

			if (EntryParams.Num() == 0)
			{
				continue;
			}

			// Determine the target DefName and DriverClassName based on parameter count
			// If 1 parameter: targets GameNetDriver by default
			// If 2+ parameters: first param is DefName, second is DriverClassName
			FName TargetDefName = (EntryParams.Num() > 1) ? FName(*EntryParams[0]) : NAME_GameNetDriver;
			FString DriverClassName = (EntryParams.Num() > 1) ? EntryParams[1] : EntryParams[0];

			// Check if this override targets GameNetDriver
			if (TargetDefName == NAME_GameNetDriver)
			{
				if (DriverClassName.Equals(ProxyNetDriverClassName, ESearchCase::IgnoreCase))
				{
					return ENetDriverOverrideState::ProxyDriverSpecified;
				}
				else
				{
					// A different driver is specified - only a conflict if cvar is enabled
					if (bProxyEnabled)
					{
						return ENetDriverOverrideState::ConflictDetected;
					}
					else
					{
						return ENetDriverOverrideState::NoOverride;
					}
				}
			}
		}

		return ENetDriverOverrideState::NoOverride;
	}
}

void FMultiServerReplicationModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
	{
		SetupProxy();
	});
}

void FMultiServerReplicationModule::SetupProxy()
{
	using namespace UE::MultiServerProxy::Private;

	// Check if -NetDriverOverrides command line argument conflicts with net.proxy.Enabled cvar
	ENetDriverOverrideState OverrideState = CheckForNetDriverOverrideConflict();

	switch (OverrideState)
	{
	case ENetDriverOverrideState::ConflictDetected:
		UE_LOGF(LogMultiServerReplication, Error,
			"Conflict detected: -NetDriverOverrides specifies a different driver for GameNetDriver, but net.proxy.Enabled cvar is set to true. " "The command line argument takes precedence. Disabling proxy mode and using the command line override.");

		bProxyEnabled = false;
		return;

	case ENetDriverOverrideState::ProxyDriverSpecified:
		bProxyEnabled = true;
		break;
	}

	// Apply the proxy override if the cvar is enabled and no conflicting command line override was found
	if (bProxyEnabled)
	{
		if (GEngine)
		{
			auto FindGameNetDriverPred = [](const FNetDriverDefinition& Def)
			{
				return Def.DefName == NAME_GameNetDriver;
			};

			FNetDriverDefinition* GameNetDriverDef = GEngine->NetDriverDefinitions.FindByPredicate(FindGameNetDriverPred);
			if (GameNetDriverDef)
			{
				GameNetDriverDef->DriverClassName = FName(TEXT("/Script/MultiServerReplication.ProxyNetDriver"));
				GameNetDriverDef->DriverClassNameFallback = FName(TEXT("/Script/MultiServerReplication.ProxyNetDriver"));
			}
		}
	}
}

bool FMultiServerReplicationModule::IsRunningAsProxy()
{
	// This function should not be called from the editor or PIE
	check(UE::GetPlayInEditorID() == INDEX_NONE);

	return UE::MultiServerProxy::Private::bProxyEnabled;
}

IMPLEMENT_MODULE(FMultiServerReplicationModule, MultiServerReplication);