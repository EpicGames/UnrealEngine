// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiServerTransport.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#include "UObject/RemoteObjectTransfer.h"
#include "Engine/EngineBaseTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiServerTransport)

DEFINE_LOG_CATEGORY_STATIC(LogMultiServerTransport, Log, All);

class FMultiServerTransportWriter : public FMemoryWriter
{
public:
	FMultiServerTransportWriter(TArray<uint8>& InBytes)
		: FMemoryWriter(InBytes)
	{
		SetUseUnversionedPropertySerialization(true);
		SetPortFlags(PPF_AvoidRemoteObjectMigration);
		ArNoDelta = true;
	}

	using FMemoryWriter::operator<<;

	virtual FArchive& operator<<(UObject*& Obj) override
	{
		FRemoteObjectReference RemoteReference { FObjectPtr(Obj) };
		RemoteReference.Serialize(*this);
		return *this;
	}

	virtual FArchive& operator<<(FObjectPtr& Value) override 
	{
		FRemoteObjectReference RemoteReference(Value);
		RemoteReference.Serialize(*this);
		return *this;
	}

	virtual FArchive& operator<<(FWeakObjectPtr& Value) override
	{
		FRemoteObjectReference RemoteReference(Value);
		RemoteReference.Serialize(*this);
		return *this;
	}

	virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return FArchiveUObject::SerializeLazyObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override { return FArchiveUObject::SerializeSoftObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FSoftObjectPath& Value) override { return FArchiveUObject::SerializeSoftObjectPath(*this, Value); }
};

class FMultiServerTransportReader : public FMemoryReader
{
public:
	FMultiServerTransportReader(TArray<uint8>& InBytes)
		: FMemoryReader(InBytes)
	{
		SetUseUnversionedPropertySerialization(true);
		SetPortFlags(PPF_AvoidRemoteObjectMigration);
		ArNoDelta = true;
	}

	using FMemoryReader::operator<<;

	virtual FArchive& operator<<(UObject*& Obj) override
	{
		FRemoteObjectReference RemoteReference;
		RemoteReference.Serialize(*this);
		Obj = RemoteReference.Resolve();
		return *this;
	}

	virtual FArchive& operator<<(FObjectPtr& Value) override 
	{
		FRemoteObjectReference RemoteReference;
		RemoteReference.Serialize(*this);
		Value = RemoteReference.ToObjectPtr();
		return *this;
	}

	virtual FArchive& operator<<(FWeakObjectPtr& Value) override
	{
		FRemoteObjectReference RemoteReference;
		RemoteReference.Serialize(*this);
		Value = RemoteReference.ToWeakPtr();
		return *this;
	}

	virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return FArchiveUObject::SerializeLazyObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override { return FArchiveUObject::SerializeSoftObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FSoftObjectPath& Value) override { return FArchiveUObject::SerializeSoftObjectPath(*this, Value); }
};

FMultiServerTransport::FMultiServerTransport(
	const FString& SharedSessionKeySecretString,
	const FString& InLocalPeerId,
	uint16 ListenPort,
	const FString& LocalServerAddress,
	const TArray<FString>& InAllServerAddresses,
	TSubclassOf<UMultiServerTransportEndpoint> EndpointClass) :
		AllServerAddresses(InAllServerAddresses),
		LocalPeerId(InLocalPeerId)
{
	UE_LOGF(LogMultiServerTransport, Display, "FMultiServerTransport initializing...");

	// store the session key bytes
	for (int i = 0; i < SharedSessionKeySize; i++)
	{
		uint8 Byte = 0;

		if (i < SharedSessionKeySecretString.Len())
		{
			Byte = static_cast<uint8>(SharedSessionKeySecretString[i]);
		}

		SharedSessionKeySecret[i] = Byte;
	}

	// Assume every server will receive the same list of addresses.
	// Sort it so all servers can deterministically figure out who else they need to connect to.
	Algo::Sort(AllServerAddresses);

	// calculate our peer index:
	LocalPeerIndex = AllServerAddresses.IndexOfByPredicate([&LocalServerAddress, ListenPort](const FString& PeerAddress)
	{
		FURL PeerURL(nullptr, ToCStr(PeerAddress), ETravelType::TRAVEL_Absolute);
		return PeerURL.Host.Equals(LocalServerAddress) && PeerURL.Port == ListenPort;
	});

	check(0 <= LocalPeerIndex && LocalPeerIndex < AllServerAddresses.Num());

	UE_LOGF(LogMultiServerTransport, Display, "FMultiServerTransport: LocalPeerIndex is %d", LocalPeerIndex);

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	check(SocketSubsystem->IsSocketWaitSupported());

	TSharedRef<FInternetAddr> ListenAddress = SocketSubsystem->CreateInternetAddr(FNetworkProtocolTypes::IPv4);
	ListenAddress->SetAnyAddress();
	ListenAddress->SetPort(ListenPort);

	ListenSocket = SocketSubsystem->CreateUniqueSocket(TEXT("Stream"), TEXT("FMultiServerTransport Listener"), false);
	ListenSocket->SetNoDelay(true);
	ListenSocket->Bind(*ListenAddress);

	// ensure the listen backlog is big enough, otherwise the socket thread entrypoint will deadlock
	// as we Connect() to each other peer in a blocking loop before we get on to the business
	// of calling Accept() for incoming connections
	ListenSocket->Listen(AllServerAddresses.Num());

	// hook up the loopback send/recv signal sockets
	{
		TSharedRef<FInternetAddr> EphemeralLoopbackAddress = SocketSubsystem->CreateInternetAddr(FNetworkProtocolTypes::IPv4);
		EphemeralLoopbackAddress->SetLoopbackAddress();
		EphemeralLoopbackAddress->SetPort(0);

		FUniqueSocket LoopbackListenSocket = SocketSubsystem->CreateUniqueSocket(TEXT("Stream"), TEXT("FMultiServerTransport LoopbackListener"), false);
		LoopbackListenSocket->SetNoDelay(true);
		LoopbackListenSocket->Bind(*EphemeralLoopbackAddress);
		LoopbackListenSocket->Listen(1);

		// get back the actual address we bound to
		LoopbackListenSocket->GetAddress(*EphemeralLoopbackAddress);

		LoopbackSendSocket = SocketSubsystem->CreateUniqueSocket(TEXT("Stream"), TEXT("FMultiServerTransport LoopbackSend"), false);
		LoopbackSendSocket->SetNoDelay(true);
		LoopbackSendSocket->Connect(*EphemeralLoopbackAddress);

		LoopbackRecvSocket = FUniqueSocket(LoopbackListenSocket->Accept(TEXT("FMultiServerTransport LoopbackRecv")));
	}

	// launch the thread
	bIsRunning = true;
	TransportThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(this, TEXT("FMultiServerTransport thread")));

	// now that the thread is running and connections are being established, we wait for it all to complete:
	{
		// first, block on all servers connecting
		UE_LOGF(LogMultiServerTransport, Display, "FMultiServerTransport initial waiting for all server connections...");
		while (!AllEndpointsConnectedEventQueue.Dequeue().IsSet())
		{
			FPlatformProcess::Sleep(0.1f);
		}
		UE_LOGF(LogMultiServerTransport, Display, "FMultiServerTransport all %d servers connected!", ConnectedEndpointCount);

		// now we're good to go!
	}

	// create all of the RPC endpoints before returning (we have to create these UObjects here on the game thread)
	for (int32 PeerIndex = 0; PeerIndex < PeerStates.Num(); PeerIndex++)
	{
		TUniquePtr<FPeer>& PeerState = PeerStates[PeerIndex];
		PeerState->Endpoint = NewObject<UMultiServerTransportEndpoint>(GetTransientPackage(), EndpointClass);
		PeerState->Endpoint->Transport = this;
		PeerState->Endpoint->PeerIndex = PeerIndex;
	}
}

FMultiServerTransport::~FMultiServerTransport()
{
	bIsRunning = false;
	SignalThreadWake();

	TransportThread->WaitForCompletion();
}

void FMultiServerTransport::CreatePendingPeerState(FUniqueSocket&& PeerSocket)
{
	UE_LOGF(LogMultiServerTransport, Verbose, "FMultiServerTransport: Creating pending peer state...");

	TUniquePtr<FPendingPeer>& NewPendingPeerState = PendingPeerStates.Emplace_GetRef(MakeUnique<FPendingPeer>());
	NewPendingPeerState->CreationTimeSeconds = FPlatformTime::Seconds();
	NewPendingPeerState->Socket = MoveTemp(PeerSocket);
}

void FMultiServerTransport::CreatePeerState(FUniqueSocket&& PeerSocket, const uint8* AuthMessageBytes, uint32 AuthMessageBytesSize)
{
	int32 PeerIndex = PeerStates.Num();

	TUniquePtr<FPeer>& NewPeerState = PeerStates.Emplace_GetRef(MakeUnique<FPeer>());
	NewPeerState->Socket = MoveTemp(PeerSocket);
	NewPeerState->SendStagingBuffer = MakeUnique<uint8[]>(FPeer::SendStagingBufferSize);
	NewPeerState->RecvStagingBuffer = MakeUnique<uint8[]>(FPeer::RecvStagingBufferSize);

	if (AuthMessageBytes)
	{
		// if we are the side doing the connecting, we are responsible for sending up the
		// magic number and the Auth block, so just stuff those directly into the staging buffer
		FMemory::Memcpy(NewPeerState->SendStagingBuffer.Get(), &MagicNumber, 4);
		NewPeerState->SendStagingBufferPutIndex += 4;

		FMemory::Memcpy(NewPeerState->SendStagingBuffer.Get() + NewPeerState->SendStagingBufferPutIndex, AuthMessageBytes, AuthMessageBytesSize);
		NewPeerState->SendStagingBufferPutIndex += AuthMessageBytesSize;
	}

	// send the higher level handshake message to the other end as the first message
	// which will have the other end consider us as fully 'connected'
	FBufferArchive HandshakeMessage;
	HandshakeMessage << LocalPeerId;
	NewPeerState->SendMessageQueue.Enqueue(MoveTemp(HandshakeMessage));
}

void FMultiServerTransport::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (TUniquePtr<FPeer>& PeerState : PeerStates)
	{
		Collector.AddReferencedObject(PeerState->Endpoint);
	}
}

FString FMultiServerTransport::GetReferencerName() const
{
	return TEXT("FMultiServerTransport");
}

static void CopyMemoryIntoSplitBuffer(uint8* DestBuffer, uint32 DestBufferSize, uint32 PutOffset, const void* SourceBuffer, uint32 SourceSize)
{
	uint32 ToCopyFirstChunkSize = 0;
	uint32 ToCopySecondChunkSize = 0;

	if ((PutOffset + SourceSize) <= DestBufferSize)
	{
		ToCopyFirstChunkSize = SourceSize;
		ToCopySecondChunkSize = 0;
	}
	else
	{
		ToCopyFirstChunkSize = DestBufferSize - PutOffset;
		ToCopySecondChunkSize = SourceSize - ToCopyFirstChunkSize;
	}

	FMemory::Memcpy(DestBuffer + PutOffset, SourceBuffer, ToCopyFirstChunkSize);

	if (ToCopySecondChunkSize > 0)
	{
		FMemory::Memcpy(DestBuffer, (const uint8*)SourceBuffer + ToCopyFirstChunkSize, ToCopySecondChunkSize);
	}
}

static void CopyMemoryFromSplitBuffer(void* DestBuffer, const uint8* SrcBuffer, uint32 SrcBufferSize, uint32 GetOffset, uint32 CopySize)
{
	uint32 ToCopyFirstChunkSize = 0;
	uint32 ToCopySecondChunkSize = 0;

	if ((GetOffset + CopySize) <= SrcBufferSize)
	{
		ToCopyFirstChunkSize = CopySize;
		ToCopySecondChunkSize = 0;
	}
	else
	{
		ToCopyFirstChunkSize = SrcBufferSize - GetOffset;
		ToCopySecondChunkSize = CopySize - ToCopyFirstChunkSize;
	}

	FMemory::Memcpy(DestBuffer, SrcBuffer + GetOffset, ToCopyFirstChunkSize);

	if (ToCopySecondChunkSize > 0)
	{
		FMemory::Memcpy((uint8*)DestBuffer + ToCopyFirstChunkSize, SrcBuffer, ToCopySecondChunkSize);
	}
}

void FMultiServerTransport::InitiateOutboundConnections()
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

	// intiate blocking Connects to every other server

	// Balance the number of client vs. server connections to minimize latency when iterating & processing each peer.

	// Rules, where n is the total number of peers:
	// If the number of peers is even:
	//  -The first half of peers in AllServerAddresses connect to ceil((n - 1) / 2) other servers.
	//  -The second half of peers in AllServerAddresses connect to floor((n - 1) / 2) other servers.
	//
	// If the number of peers is odd:
	//  -All peers connect to (n - 1) / 2 servers.
	//
	// All connections are made starting with the index directly above the local peer's own in the sorted list,
	// wrapping around.
	//
	// For example, when there are 4 peers, each one would make connections as follows:
	// 1 -> 2 3
	// 2 -> 3 4
	// 3 -> 4
	// 4 -> 1
	// 
	// Or with 7 peers:
	// 1 -> 2 3 4
	// 2 -> 3 4 5
	// 3 -> 4 5 6
	// 4 -> 5 6 7
	// 5 -> 6 7 1
	// 6 -> 7 1 2
	// 7 -> 1 2 3
	//
	const int32 CeilConnections = FMath::CeilToInt((AllServerAddresses.Num() - 1) / 2.0f);
	const int32 FloorConnections = FMath::FloorToInt((AllServerAddresses.Num() - 1) / 2.0f);

	// When the number of total peers is even, peers in the first half of the list will connect to one more
	// peer than those in the second half of the list.
	// When the number of total peers is odd, all peers will connect to the same number of other peers.
	const int32 NumToConnectTo = LocalPeerIndex < (AllServerAddresses.Num() / 2) ? CeilConnections : FloorConnections;

	TSharedRef<FInternetAddr> RemoteAddress = SocketSubsystem->CreateInternetAddr(FNetworkProtocolTypes::IPv4);

	FRandomStream RandomStream;
	RandomStream.GenerateNewSeed();

	for (int32 RemotePeerCount = 0; RemotePeerCount < NumToConnectTo; ++RemotePeerCount)
	{
		const int32 RemotePeerIndex = (LocalPeerIndex + RemotePeerCount + 1) % AllServerAddresses.Num();
		UE_LOGF(LogMultiServerTransport, Display, "FMultiServerTransport: Initiating connection to RemotePeerIndex %d...", RemotePeerIndex);

		bool bIsAddressValid = false;
		RemoteAddress->SetIp(*AllServerAddresses[RemotePeerIndex], bIsAddressValid);
		check(bIsAddressValid);

		for (;;)
		{
			FUniqueSocket Socket = SocketSubsystem->CreateUniqueSocket(TEXT("Stream"), TEXT("FMultiServerTransport PeerSocket"), false);
			checkf(Socket, TEXT("FMultiServerTransport PeerSocket creation failed"));

			Socket->SetNoDelay(true);

			if(Socket->Connect(*RemoteAddress))
			{
				union
				{
					uint8 Nonce[NonceSize];
					struct { uint32 NonceA; uint32 NonceB; uint32 NonceC; };
				};

				static_assert(NonceSize == 12);
				NonceA = RandomStream.GetUnsignedInt();
				NonceB = RandomStream.GetUnsignedInt();
				NonceC = RandomStream.GetUnsignedInt();
				
				// when we initiate a connection, we are responsible for sending the magic number and initial auth packet
				uint8 AuthMessageBytes[AuthMessageSize];

				// the first NonceSize bytes are the random nonce
				FMemory::Memcpy(&AuthMessageBytes[0], &Nonce[0], NonceSize);

				// the last bytes are the HMAC SHA1
				static_assert(HMACHashSize == 20);
				FSHA1::HMACBuffer(SharedSessionKeySecret, SharedSessionKeySize, &Nonce[0], NonceSize, &AuthMessageBytes[AuthMessageSize - HMACHashSize]);

				// create the peer state and stage this directly in
				CreatePeerState(MoveTemp(Socket), AuthMessageBytes, AuthMessageSize);
				break;
			}
			else
			{
				UE_LOGF(LogMultiServerTransport, Warning, "FMultiServerTransport: Connection to RemotePeerIndex %d failed, retrying...", RemotePeerIndex);
			}

			FPlatformProcess::Sleep(0.1f);
		}
	}
}

FMultiServerTransport::FPendingPeer::EPendingPeerTickResult FMultiServerTransport::FPendingPeer::TickPendingPeer(const uint8* InSharedSessionKeySecret)
{
	// timeout?
	double NowTime = FPlatformTime::Seconds();

	bool bTimedOut = (NowTime - CreationTimeSeconds) > ConnectTimeout;
	bool bIsValid = Socket.IsValid();
	bool bPromoteToPeer = false;

	// validate the magic number?
	if (0 < MagicNumberOffset && MagicNumberOffset <= 4)
	{
		// if we have received any bytes at all we can at least partially validate the magic number (this allows
		// us to quickly reject invalid clients as soon as they send us anything at all)
		if (FMemory::Memcmp(&MagicNumberBuffer, &MagicNumber, MagicNumberOffset) != 0)
		{
			bIsValid = false;
			UE_LOGF(LogMultiServerTransport, Warning, "FMultiServerTransport: Pending Peer magic number BAD (offset %u)", MagicNumberOffset);
		}
	}
			
	if (MagicNumberOffset == 4)
	{
		// we have received the full magic number and validated it, now check out
		// the auth block
		if(AuthMessageBufferOffset == AuthMessageSize)
		{
			// validate the nonce
			static_assert(HMACHashSize == 20);
			uint8 HmacToValidate[HMACHashSize];
			FSHA1::HMACBuffer(InSharedSessionKeySecret, SharedSessionKeySize, &AuthMessageBuffer[0], NonceSize, &HmacToValidate[0]);

			if (FMemory::Memcmp(&HmacToValidate, &AuthMessageBuffer[AuthMessageSize - HMACHashSize], HMACHashSize) != 0)
			{
				bIsValid = false;
				UE_LOGF(LogMultiServerTransport, Warning, "FMultiServerTransport: NONCE FAILED");
			}
			else
			{
				bPromoteToPeer = true;
			}
		}
	}

	if (bPromoteToPeer)
	{
		return EPendingPeerTickResult::PromoteToPeer;
	}

	if (!bIsValid)
	{
		return EPendingPeerTickResult::IsInvalid;
	}

	if (bTimedOut)
	{
		return EPendingPeerTickResult::TimedOut;
	}

	return EPendingPeerTickResult::StillPending;
}

void FMultiServerTransport::FPendingPeer::HandlePendingPeerRecv()
{
	// are we reading the magic number or the auth message?
	uint8* RecvBuffer = nullptr;
	uint32* RecvOffset = nullptr;
	int32 RecvSize = 0;
	if (MagicNumberOffset < 4)
	{
		// magic number
		RecvOffset = &MagicNumberOffset;
		RecvBuffer = (uint8*)&MagicNumberBuffer;
		RecvSize = 4 - *RecvOffset;
	}
	else
	{
		// auth
		RecvOffset = &AuthMessageBufferOffset;
		RecvBuffer = (uint8*)&AuthMessageBuffer;
		RecvSize = AuthMessageSize - *RecvOffset;
	}

	int32 BytesRead = 0;
	if (Socket->Recv(RecvBuffer + *RecvOffset, RecvSize, BytesRead, ESocketReceiveFlags::None) && (BytesRead > 0))
	{
		check(BytesRead <= RecvSize);
		*RecvOffset += BytesRead;
	}
	else
	{
		// disconnected, dump it (will get cleaned up next iteration)
		Socket = nullptr;
	}
}

void FMultiServerTransport::FPeer::StageOutgoingMessages()
{
	// stage as much as possible
	for (;;)
	{
		uint32 StagingBufferUsedSpace = (SendStagingBufferPutIndex - SendStagingBufferGetIndex + SendStagingBufferSize) % SendStagingBufferSize;
		uint32 StagingBufferFreeSpace = (SendStagingBufferSize - 1) - StagingBufferUsedSpace;
			
		if ((StagingBufferFreeSpace > 0) && (!SendMessageQueue.IsEmpty()))
		{
			TArray<uint8>* MessageToStage = nullptr;
			if (SendStagingBufferPendingMessageOffset >= 0)
			{
				// we have already started staging the first message, and there is some free space so let's go
				MessageToStage = SendMessageQueue.Peek();
			}
			else
			{
				// do we have enough room in the buffer for the 4 byte size? If so, we can start staging this
				// message. Otherwise, we come back later.
				if (StagingBufferFreeSpace >= 4)
				{
					MessageToStage = SendMessageQueue.Peek();

					uint32 MessageSize = MessageToStage->Num();

					CopyMemoryIntoSplitBuffer(SendStagingBuffer.Get(), SendStagingBufferSize, SendStagingBufferPutIndex, &MessageSize, 4);
					SendStagingBufferPendingMessageOffset = 0;

					SendStagingBufferPutIndex = (SendStagingBufferPutIndex + 4) % SendStagingBufferSize;
					StagingBufferFreeSpace -= 4;
					StagingBufferUsedSpace += 4;
				}
				else
				{
					// can't stage any more
					break;
				}
			}

			if (MessageToStage)
			{
				uint32 MessageSizeRemaining = MessageToStage->Num() - SendStagingBufferPendingMessageOffset;

				uint32 MessageSizeToStage = MessageSizeRemaining;
				if (MessageSizeToStage > StagingBufferFreeSpace)
				{
					MessageSizeToStage = StagingBufferFreeSpace;
				}

				if (MessageSizeToStage > 0)
				{
					CopyMemoryIntoSplitBuffer(
						SendStagingBuffer.Get(), SendStagingBufferSize, SendStagingBufferPutIndex,
						&(*MessageToStage)[SendStagingBufferPendingMessageOffset], MessageSizeToStage);

					SendStagingBufferPutIndex = (SendStagingBufferPutIndex + MessageSizeToStage) % SendStagingBufferSize;
					SendStagingBufferPendingMessageOffset += MessageSizeToStage;

					if (SendStagingBufferPendingMessageOffset == MessageToStage->Num())
					{
						// we staged the whole message
						SendStagingBufferPendingMessageOffset = -1;

						// pop it out of the queue
						SendMessageQueue.Dequeue();
					}
				}
			}
		}
		else
		{
			// can't stage any more
			break;
		}
	}
}

bool FMultiServerTransport::FPeer::HandlePeerSend()
{
	// send stuff out from the staging buffer
	uint32 SizeToSend = 0;

	// if the get and put are equal, we shouldn't have requested it be writable
	check(SendStagingBufferPutIndex != SendStagingBufferGetIndex);

	if (SendStagingBufferPutIndex > SendStagingBufferGetIndex)
	{
		// no wrap
		SizeToSend = SendStagingBufferPutIndex - SendStagingBufferGetIndex;
	}
	else
	{
		// wrap, we can only make a single Send() so just send up to the end of the buffer
		SizeToSend = SendStagingBufferSize - SendStagingBufferGetIndex;
	}

	int32 BytesSent = 0;
	if (!Socket->Send(&SendStagingBuffer[SendStagingBufferGetIndex], SizeToSend, BytesSent))
	{
		UE_LOGF(LogMultiServerTransport, Warning, "FMultiServerTransport socket send failed (endpoint disconnection) RemotePeerId=%ls", *RemotePeerId);
		return false;
	}

	SendStagingBufferGetIndex = (SendStagingBufferGetIndex + BytesSent) % SendStagingBufferSize;
	return true;
}

FMultiServerTransport::FPeer::EPeerRecvResult FMultiServerTransport::FPeer::HandlePeerRecv()
{
	uint32 StagingBufferUsedSpace = (RecvStagingBufferPutIndex - RecvStagingBufferGetIndex + RecvStagingBufferSize) % RecvStagingBufferSize;
	uint32 StagingBufferFreeSpace = (RecvStagingBufferSize - 1) - StagingBufferUsedSpace;

	// Read from socket into the circular staging buffer.
	// We can only offer a single contiguous region to Recv(), so if Put is
	// near the end of the buffer we recv up to the end, and the wraparound
	// portion will be picked up on the next iteration.
	uint32 RecvSize = FMath::Min(StagingBufferFreeSpace, RecvStagingBufferSize - RecvStagingBufferPutIndex);

	if (RecvSize > 0)
	{
		int32 BytesRead = 0;
		if (!Socket->Recv(RecvStagingBuffer.Get() + RecvStagingBufferPutIndex, RecvSize, BytesRead, ESocketReceiveFlags::None))
		{
			UE_LOGF(LogMultiServerTransport, Warning, "FMultiServerTransport socket recv failed (endpoint disconnection) RemotePeerId=%ls", *RemotePeerId);
			return EPeerRecvResult::Disconnected;
		}
		else
		{
			RecvStagingBufferPutIndex = (RecvStagingBufferPutIndex + BytesRead) % RecvStagingBufferSize;
			StagingBufferUsedSpace += BytesRead;
			StagingBufferFreeSpace -= BytesRead;
		}
	}

	bool bHandshakeReceived = false;

	// Assemble as many complete messages as possible from the staging buffer
	for (;;)
	{
		if (RecvMessageBuffer.Num() == 0)
		{
			// We need a 4-byte message size header
			if (StagingBufferUsedSpace < 4)
			{
				break;
			}

			uint32 MessageSize = 0;
			CopyMemoryFromSplitBuffer(&MessageSize, RecvStagingBuffer.Get(), RecvStagingBufferSize, RecvStagingBufferGetIndex, 4);
			RecvStagingBufferGetIndex = (RecvStagingBufferGetIndex + 4) % RecvStagingBufferSize;
			StagingBufferUsedSpace -= 4;

			RecvMessageBuffer.SetNumUninitialized(MessageSize);
			RecvMessageBufferOffset = 0;
		}

		// Copy body bytes from circular staging buffer into message buffer
		uint32 BodyBytesNeeded = RecvMessageBuffer.Num() - RecvMessageBufferOffset;
		uint32 ToCopy = FMath::Min(BodyBytesNeeded, StagingBufferUsedSpace);
		if (ToCopy > 0)
		{
			CopyMemoryFromSplitBuffer(
				RecvMessageBuffer.GetData() + RecvMessageBufferOffset,
				RecvStagingBuffer.Get(), RecvStagingBufferSize,
				RecvStagingBufferGetIndex, ToCopy);
			RecvStagingBufferGetIndex = (RecvStagingBufferGetIndex + ToCopy) % RecvStagingBufferSize;
			RecvMessageBufferOffset += ToCopy;
			StagingBufferUsedSpace -= ToCopy;
		}

		if (RecvMessageBufferOffset == (uint32)RecvMessageBuffer.Num())
		{
			// Complete message assembled
			if (RemotePeerId.IsEmpty())
			{
				// this is the initial handshake - deserialize the remote peer id and signal
				// the caller so it can update transport-level connection state
				FMemoryReader HandshakeMessage(RecvMessageBuffer);
				HandshakeMessage << RemotePeerId;

				bHandshakeReceived = true;
			}
			else
			{
				// push it into the queue for the game thread
				RecvMessageQueue.Enqueue(MoveTemp(RecvMessageBuffer));
			}

			// reset for next message
			RecvMessageBuffer.Reset();
			RecvMessageBufferOffset = 0;
		}
		else
		{
			break; // need more data for body
		}
	}

	return bHandshakeReceived ? EPeerRecvResult::HandshakeReceived : EPeerRecvResult::Ok;
}

uint32 FMultiServerTransport::Run()
{
	UE_LOGF(LogMultiServerTransport, Display, "FMultiServerTransport: Thread startup");

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

	InitiateOutboundConnections();

	if (AllServerAddresses.Num() <= 1)
	{
		AllEndpointsConnectedEventQueue.Enqueue(true);
	}

	TArray<FSocket*> Sockets;
	TArray<ISocketSubsystem::ESocketStateFlags> InOutSocketStates;

	while (bIsRunning)
	{
		//
		// tick pending peers
		//
		for (int32 PendingPeerIndex = 0; PendingPeerIndex < PendingPeerStates.Num();)
		{
			TUniquePtr<FPendingPeer>& PendingPeerState = PendingPeerStates[PendingPeerIndex];

			FPendingPeer::EPendingPeerTickResult TickResult = PendingPeerState->TickPendingPeer(SharedSessionKeySecret);

			if (TickResult == FPendingPeer::EPendingPeerTickResult::PromoteToPeer)
			{
				UE_LOGF(LogMultiServerTransport, Verbose, "FMultiServerTransport: PROMOTING TO PEER");
				CreatePeerState(MoveTemp(PendingPeerState->Socket), nullptr, 0);

				// dump it
				PendingPeerStates.RemoveAt(PendingPeerIndex);
			}
			else if ((TickResult == FPendingPeer::EPendingPeerTickResult::TimedOut) || (TickResult == FPendingPeer::EPendingPeerTickResult::IsInvalid))
			{
				PendingPeerStates.RemoveAt(PendingPeerIndex);
			}
			else
			{
				check(TickResult == FPendingPeer::EPendingPeerTickResult::StillPending);
				PendingPeerIndex++;
			}
		}

		//
		// pop messages to send into the staging buffers
		//
		for (int32 PeerIndex = 0; PeerIndex < PeerStates.Num(); PeerIndex++)
		{
			TUniquePtr<FPeer>& PeerState = PeerStates[PeerIndex];
			PeerState->StageOutgoingMessages();
		}

		//
		// gather all of the sockets we care about
		//

		Sockets.Reset();
		InOutSocketStates.Reset();

		//
		// add the pending peers (only trying to recv from them)
		//
		const int32 PendingPeerCount = PendingPeerStates.Num();
		for (int32 PendingPeerIndex = 0; PendingPeerIndex < PendingPeerStates.Num(); PendingPeerIndex++)
		{
			TUniquePtr<FPendingPeer>& PendingPeerState = PendingPeerStates[PendingPeerIndex];

			uint8 RequestFlags = 0;
			RequestFlags |= (uint8)ISocketSubsystem::ESocketStateFlags::Readable;
			RequestFlags |= (uint8)ISocketSubsystem::ESocketStateFlags::Error;

			Sockets.Add(PendingPeerState->Socket.Get());
			InOutSocketStates.Add((ISocketSubsystem::ESocketStateFlags)RequestFlags);
		}

		//
		// add the connected peers (query recv always, query send if bytes are staged)
		//
		const int32 PeerCount = PeerStates.Num();
		for (int32 PeerIndex = 0; PeerIndex < PeerStates.Num(); PeerIndex++)
		{
			TUniquePtr<FPeer>& PeerState = PeerStates[PeerIndex];

			uint8 RequestFlags = 0;
			RequestFlags |= (uint8)ISocketSubsystem::ESocketStateFlags::Readable;
			RequestFlags |= (uint8)ISocketSubsystem::ESocketStateFlags::Error;

			if (PeerState->SendStagingBufferPutIndex != PeerState->SendStagingBufferGetIndex)
			{
				RequestFlags |= (uint8)ISocketSubsystem::ESocketStateFlags::Writable;
			}

			Sockets.Add(PeerState->Socket.Get());
			InOutSocketStates.Add((ISocketSubsystem::ESocketStateFlags)RequestFlags);
		}

		//
		// if we are still waiting for connections, add the listen socket (for new incoming connections)
		//
		uint32 ListenSocketIndex = 0;
		if (ListenSocket)
		{
			ListenSocketIndex = Sockets.Num();
			Sockets.Add(ListenSocket.Get());

			uint8 RequestFlags = 0;
			RequestFlags |= (uint8)ISocketSubsystem::ESocketStateFlags::Readable;
			RequestFlags |= (uint8)ISocketSubsystem::ESocketStateFlags::Error;
			InOutSocketStates.Add((ISocketSubsystem::ESocketStateFlags)RequestFlags);
		}

		//
		// add the thread wake signal
		//
		uint32 ThreadWakeSocketIndex = Sockets.Num();
		{
			Sockets.Add(LoopbackRecvSocket.Get());

			uint8 RequestFlags = 0;
			RequestFlags |= (uint8)ISocketSubsystem::ESocketStateFlags::Readable;
			RequestFlags |= (uint8)ISocketSubsystem::ESocketStateFlags::Error;
			InOutSocketStates.Add((ISocketSubsystem::ESocketStateFlags)RequestFlags);
		}

		//
		// query sockets, blocking forever
		//
		SocketSubsystem->QuerySocketStateMany(Sockets, InOutSocketStates, FTimespan(-1));

		//
		// handle socket query results
		//
		for (int32 SocketIndex = 0; SocketIndex < Sockets.Num(); SocketIndex++)
		{
			FSocket* Socket = Sockets[SocketIndex];
			uint8 SocketState = (uint8)InOutSocketStates[SocketIndex];

			if (SocketState != 0)
			{
				if (SocketIndex < PendingPeerCount)
				{
					//
					// pending peer
					//
					TUniquePtr<FPendingPeer>& PendingPeerState = PendingPeerStates[SocketIndex];

					if (SocketState & (uint8)ISocketSubsystem::ESocketStateFlags::Error)
					{
						UE_LOGF(LogMultiServerTransport, Verbose, "FMultiServerTransport: error on pending socket %d", SocketIndex);

						// dump it (will get cleaned up next iteration)
						PendingPeerState->Socket = nullptr;
					}
					else if (SocketState & (uint8)ISocketSubsystem::ESocketStateFlags::Readable)
					{
						PendingPeerState->HandlePendingPeerRecv();
					}
				}
				else if (SocketIndex < PendingPeerCount + PeerCount)
				{
					//
					// peer
					//
					TUniquePtr<FPeer>& PeerState = PeerStates[SocketIndex - PendingPeerCount];

					if (SocketState & (uint8)ISocketSubsystem::ESocketStateFlags::Error)
					{
						UE_LOGF(LogMultiServerTransport, Warning, "FMultiServerTransport detected disconnection of peer");
						bIsRunning = false;
					}

					if (SocketState & (uint8)ISocketSubsystem::ESocketStateFlags::Writable)
					{
						if (!PeerState->HandlePeerSend())
						{
							bIsRunning = false;
						}
					}

					if (SocketState & (uint8)ISocketSubsystem::ESocketStateFlags::Readable)
					{
						FPeer::EPeerRecvResult RecvResult = PeerState->HandlePeerRecv();
						if (RecvResult == FPeer::EPeerRecvResult::Disconnected)
						{
							bIsRunning = false;
						}
						else if (RecvResult == FPeer::EPeerRecvResult::HandshakeReceived)
						{
							ConnectedEndpointCount++;
							if (ConnectedEndpointCount == AllServerAddresses.Num() - 1)
							{
								// all endpoints are established - tear down the listen socket
								UE_LOGF(LogMultiServerTransport, Display, "FMultiServerTransport: All %d endpoints connected, tearing down listen socket", ConnectedEndpointCount);
								ListenSocket = nullptr;

								// and unblock the main thread
								AllEndpointsConnectedEventQueue.Enqueue(true);
							}
						}
					}
				}
				else if (ListenSocket && (SocketIndex == ListenSocketIndex))
				{
					check (Socket == ListenSocket.Get());

					if (SocketState & (uint8)ISocketSubsystem::ESocketStateFlags::Readable)
					{
						UE_LOGF(LogMultiServerTransport, Display, "FMultiServerTransport: Accepted connection...");

						// the listen socket has a connection request, accept it as a pending peer
						FSocket* NewPeerSocket = ListenSocket->Accept(TEXT("FMultiServerTransport PeerSocket"));
						CreatePendingPeerState(FUniqueSocket(NewPeerSocket));
					}
				}
				else if (SocketIndex == ThreadWakeSocketIndex)
				{
					check (Socket == LoopbackRecvSocket.Get());

					// drain the wake socket - just in case we've been woken up multiple times, try
					// to drain a bunch of signals at a time
					uint8 TempSignals[32];

					if (SocketState & (uint8)ISocketSubsystem::ESocketStateFlags::Readable)
					{
						int32 BytesRead = 0;
						if (!Socket->Recv(TempSignals, 32, BytesRead, ESocketReceiveFlags::None))
						{
							UE_LOGF(LogMultiServerTransport, Fatal, "FMultiServerTransport unexpected recv error on LoopbackRecvSocket.");
						}
					}
				}
			}
		}
	}

	return 0;
}

void FMultiServerTransport::SignalThreadWake()
{
	// write the byte
	uint8 TheByte = 0xEE;

	int32 BytesSent = 0;
	LoopbackSendSocket->Send(&TheByte, 1, BytesSent);

	check(BytesSent == 1);
}

UMultiServerTransportEndpoint* FMultiServerTransport::GetEndpointForRemotePeer(FStringView RemotePeerId) const
{
	for (int32 PeerIndex = 0; PeerIndex < PeerStates.Num(); PeerIndex++)
	{
		const TUniquePtr<FPeer>& PeerState = PeerStates[PeerIndex];
		if (RemotePeerId.Equals(PeerState->RemotePeerId, ESearchCase::IgnoreCase))
		{
			return PeerState->Endpoint;
		}
	}

	return nullptr;
}

void FMultiServerTransport::ForEachEndpoint(TFunctionRef<void(UMultiServerTransportEndpoint*)> Operation) const
{
	for (int32 PeerIndex = 0; PeerIndex < PeerStates.Num(); PeerIndex++)
	{
		const TUniquePtr<FPeer>& PeerState = PeerStates[PeerIndex];
		Operation(PeerState->Endpoint);
	}
}

bool FMultiServerTransport::GameThreadTick()
{
	const uint32 AllocaBufferSize = 1024;
	uint8* AllocaBuffer = static_cast<uint8*>(FMemory_Alloca(AllocaBufferSize));

	// dequeue messages from the endpoints and execute the RPCs
	for (int32 PeerIndex = 0; PeerIndex < PeerStates.Num(); PeerIndex++)
	{
		const TUniquePtr<FPeer>& PeerState = PeerStates[PeerIndex];
		TObjectPtr<UMultiServerTransportEndpoint> PeerEndpoint = PeerState->Endpoint;

		for (;;)
		{
			TOptional<TArray<uint8>> IncomingRPC = PeerState->RecvMessageQueue.Dequeue();
			if (!IncomingRPC.IsSet())
			{
				break;
			}

			// dispatch RPC to the endpoint
			FMultiServerTransportReader RPCMessage(IncomingRPC.GetValue());

			FName FunctionName;
			RPCMessage << FunctionName;

			UFunction* Function = PeerEndpoint->FindFunctionChecked(FunctionName);

			// allocate params
			uint32 ParmsSize = Function->ParmsSize;
			bool bAllocParms = ParmsSize > AllocaBufferSize;
			uint8* Parms = bAllocParms ? static_cast<uint8*>(FMemory::Malloc(ParmsSize)) : AllocaBuffer;
			FMemory::Memzero(Parms, Function->ParmsSize);

			// construct
			Function->InitializeStruct(Parms);

			// deserialize
			for (TFieldIterator<FProperty> ParamIt(Function); ParamIt && (ParamIt->PropertyFlags & CPF_Parm); ++ParamIt)
			{
				FProperty* Param = *ParamIt;

				const bool bIsInput = !Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm);
				if (bIsInput)
				{
					void* PropertyPtr = (uint8*)Parms + Param->GetOffset_ForUFunction();
					Param->SerializeItem(FStructuredArchiveFromArchive(RPCMessage).GetSlot(), PropertyPtr);
				}
			}

			// execute
			UE::Net::Private::FScopedRemoteRPCMode ReceivingRemoteRPC(Function, UE::Net::Private::ERemoteFunctionMode::Receiving);
			PeerEndpoint->ProcessEvent(Function, Parms);

			// destruct
			Function->DestroyStruct(Parms);

			if (bAllocParms)
			{
				FMemory::Free(Parms);
			}
		}
	}

	return bIsRunning.Load();
}

void FMultiServerTransport::SendToPeer(int32 PeerIndex, TArray<uint8>&& Message)
{
	TUniquePtr<FPeer>& PeerState = PeerStates[PeerIndex];

	PeerState->SendMessageQueue.Enqueue(MoveTemp(Message));
	SignalThreadWake();
}

FString UMultiServerTransportEndpoint::GetLocalPeerId() const
{
	return Transport->LocalPeerId;
}

FString UMultiServerTransportEndpoint::GetRemotePeerId() const
{
	TUniquePtr<FMultiServerTransport::FPeer>& PeerState = Transport->PeerStates[PeerIndex];

	return PeerState->RemotePeerId;
}

int32 UMultiServerTransportEndpoint::GetFunctionCallspace( UFunction* Function, FFrame* Stack )
{
	if (!(Function->FunctionFlags & FUNC_Net))
	{
		// Not a network function
		return FunctionCallspace::Local;
	}

	// it is a network function - are we the sender or receiver?
	if (UE::Net::Private::FCoreNetContext::Get()->GetCurrentRemoteFunctionMode() == UE::Net::Private::ERemoteFunctionMode::Receiving)
	{
		// we are the receiver - execute it locally
		return FunctionCallspace::Local;
	}

	// else, execute it remotely
	return FunctionCallspace::Remote;
}

bool UMultiServerTransportEndpoint::CallRemoteFunction( UFunction* Function, void* Parms, struct FOutParmRec* OutParms, FFrame* Stack )
{
	TArray<uint8> RPCMessageBuffer;
	FMultiServerTransportWriter RPCMessage(RPCMessageBuffer);

	FName FunctionName = Function->GetFName();
	RPCMessage << FunctionName;

	for (TFieldIterator<FProperty> ParamIt(Function); ParamIt && (ParamIt->PropertyFlags & CPF_Parm); ++ParamIt)
	{
		FProperty* Param = *ParamIt;

		const bool bIsInput = !Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm);
		if (bIsInput)
		{
			void* PropertyPtr = (uint8*)Parms + Param->GetOffset_ForUFunction();
			Param->SerializeItem(FStructuredArchiveFromArchive(RPCMessage).GetSlot(), PropertyPtr);
		}
	}

	// send it out!
	UE_AUTORTFM_ONCOMMIT(this, RPCMessageBuffer)
	{
		Transport->SendToPeer(PeerIndex, MoveTemp(RPCMessageBuffer));
	};

	return true;
}
