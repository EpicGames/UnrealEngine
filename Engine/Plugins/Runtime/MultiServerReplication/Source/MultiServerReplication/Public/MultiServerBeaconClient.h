// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconClient.h"
#include "MultiServerBeaconClient.generated.h"

class UMultiServerNode;
struct FUpdateLevelVisibilityLevelInfo;

DECLARE_DELEGATE(FOnMultiServerConnectionEstablished);

/**
 * An online beacon that helps manage connecting to MultiServer Networks, and replicating
 * metadata about the MultiServer Network.
 *
 * This Actor will exist on All MultiServer Nodes, and multiple MultiServerBeaconClient Actors may exist on non-Client Nodes,
 * one for each other connected node.
 */
UCLASS(transient, config=Engine, notplaceable)
class MULTISERVERREPLICATION_API AMultiServerBeaconClient : public AOnlineBeaconClient
{
	GENERATED_BODY()

public:
	AMultiServerBeaconClient();

//~ Begin AOnlineBeaconClient interface
	virtual void DestroyBeacon() override;
	virtual void OnConnected() override;
	virtual void OnFailure() override;
//~ End AOnlineBeaconClient interface

//~ Begin AOnlineBeacon interface
	virtual bool InitBase() override;
//~ End AOnlineBeacon interface

	/**
	 * Attempt to connect to another specified MultiServer node.
	 * AOnlineBeaconClient::OnFailure will be called immediately if there's a problem
	 * within this call.
	 *
	 * @param ConnectionInfo	Information 
	 */
	virtual void ConnectToServer(const FString& ConnectInfo);

	void SetOwningNode(UMultiServerNode* InOwningNode) { OwningNode = InOwningNode; }

	UFUNCTION(Reliable, Server, WithValidation)
	virtual void ServerUpdateLevelVisibility(const FUpdateLevelVisibilityLevelInfo& LevelVisibility);

	UFUNCTION(Reliable, Server, WithValidation, SealedEvent)
	void ServerUpdateMultipleLevelsVisibility(const TArray<FUpdateLevelVisibilityLevelInfo>& LevelVisibilities);

	/** The Id on the other side of the connection */
	FString GetRemotePeerId() const;

	/** The Id on the local side of the connection */
	FString GetLocalPeerId() const;

	/**
	 * Actor Role on client beacons stays as authority, so this function can be used to determine whether this instance is acting as an authority.
	 * Currently, can be used to know whether a Client or Server RPC should be called from this instance.
	 * But once a new RPC type that supports both clients and servers simultaneously is added, this concept will be
	 * abstracted and we shouldn't need this function anymore.
	 */
	bool IsAuthorityBeacon() const;

protected:
	UPROPERTY()
	FString RemotePeerId;

	virtual void OnLevelRemovedFromWorld(class ULevel* Level, class UWorld* World);
	virtual void OnLevelAddedToWorld(class ULevel* Level, class UWorld* World);

	FDelegateHandle OnLevelRemovedFromWorldHandle;
	FDelegateHandle OnLevelAddedToWorldHandle;

	FName NetworkRemapPath(FName InPackageName, bool bReading);

private:
	friend class AMultiServerBeaconHostObject;

	UFUNCTION(Client, Reliable)
	void ClientPeerConnected(const FString& NewRemotePeerId, AMultiServerBeaconClient* Beacon);

	UFUNCTION(Reliable, Server)
	void ServerSetRemotePeerId(const FString& NewRemotePeerId);

	UPROPERTY(Transient)
	TObjectPtr<UMultiServerNode> OwningNode;
};
