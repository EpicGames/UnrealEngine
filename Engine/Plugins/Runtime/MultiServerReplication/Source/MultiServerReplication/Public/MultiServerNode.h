// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "MultiServerNode.generated.h"

class AActor;
class APlayerController;
class AGameSession;
class UNetDriver;

class AMultiServerBeaconHost;
class AMultiServerBeaconHostObject;
class AMultiServerBeaconClient;
class UMultiServerPeerConnection;

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnMultiServerConnectedEvent, const FString&, const FString&, AMultiServerBeaconClient*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAllServersConnectedEvent, UMultiServerNode*);

/* Parameters for initializing a UMultiServerNode */
struct FMultiServerNodeCreateParams
{
	/** World in which to create the node */
	UWorld* World = nullptr;

	/** String identifier of this node. Must be unique among all nodes that will connect to each other. */
	FString LocalPeerId;

	/** The local IP address of this node, only used so it can find itself in the PeerAddresses list. */
	FString ListenIp;

	/** The port on which this node will listen for new connections. */ 
	uint16 ListenPort = 0;

	/** The number of servers we're expecting in the cluster */
	uint32 NumServers = 1;

	/** List of addresses of other nodes to attempt to connect to. */
	TArray<FString> PeerAddresses;

	/** Beacon client class that will be instantiated for each connection. Can implement its own RPCs. */
	TSubclassOf<AMultiServerBeaconClient> UserBeaconClass;

	/** Callback invoked when a connection to a remote node is established. */
	FOnMultiServerConnectedEvent OnMultiServerConnected;
};

/**
 * The MultiServer node is a system / control scheme for connecting multiple dedicated server
 * processes to each other and allowing them to communicate via online beacons.
 * 
 * The basic usage pattern is to create a UMultiServerNode in project code via UMultiServerNode::Create.
 * For a typical UE game, a good place might be in an AGameSession subclass in the RegisterServer override.
 * The node manages all the connections to other servers, and will attempt to establish them upon creation
 * based on the PeerAddresses in the FMultiServerNodeCreateParams.
 * 
 * The main user-extension point is to subclass AMultiServerBeaconClient. This subclass can implement its own
 * RPCs to send custom messages to other servers connected to the node. The OnMultiServerConnected callback in the
 * FMultiServerNodeCreateParams will be called when a new connection is established, with the instance of the user
 * beacon as an argument. The user code can call its RPCs on the instance.
 */
UCLASS(Config=Engine, Transient, DisplayName = "MultiServer Node")
class MULTISERVERREPLICATION_API UMultiServerNode : public UObject
{
	GENERATED_BODY()

public:
	UMultiServerNode();

	static UMultiServerNode* Create(const FMultiServerNodeCreateParams& Params);

	static void ParseCommandLineIntoCreateParams(FMultiServerNodeCreateParams& InOutParams);

	virtual void BeginDestroy() override;

	/** @return true if all of the servers we are expecting have connected and registered with this node */
	bool AreAllServersConnected() const;

	bool RegisterServer(const FMultiServerNodeCreateParams& Params);

	AMultiServerBeaconClient* GetBeaconClientForRemotePeer(FStringView RemotePeerId) const;

	template<class T>
	T* GetBeaconClientForRemotePeer(FStringView RemotePeerId) const;

	AMultiServerBeaconClient* GetBeaconClientForURL(const FString& InURL) const;

	template<class T>
	T* GetBeaconClientForURL(const FString& InURL) const;

	FString GetLocalPeerId() const { return LocalPeerId; }

	float GetRetryConnectDelay() const { return RetryConnectDelay; }
	float GetRetryConnectMaxDelay() const { return RetryConnectMaxDelay; }

	TSubclassOf<AMultiServerBeaconClient> GetUserBeaconClass() const { return UserBeaconClass; }

	void ForEachBeaconClient(TFunctionRef<void(AMultiServerBeaconClient*)> Operation) const;

	void ForEachNetDriver(TFunctionRef<void(UNetDriver*)> Operation) const;

	/** How many acknolwedged connections do we have? */
	uint32 GetConnectionCount() const;

	/** How many connections are we expecting? */
	uint32 GetNumExpectedConnections() const { return (NumExpectedServers > 0) ? NumExpectedServers - 1 : 0; }

	/** Notify this MultiServerNode that a Beacon has connected to it */
	void NotifyMultiServerConnected(const FString& LocalServerId, const FString& ConnectedRemoteServerId, AMultiServerBeaconClient* ConnectionToServer);

	/** Allow users to register a delegate for when all servers are connected */
	TMulticastDelegateRegistration<void(UMultiServerNode*)>& OnAllServersConnected()
	{
		return OnAllServersConnectedEvent;
	}

	void BroadcastAllServersConnected()
	{
		OnAllServersConnectedEvent.Broadcast(this);
	}

private:
	FOnAllServersConnectedEvent OnAllServersConnectedEvent;

	FOnMultiServerConnectedEvent OnMultiServerConnectedEvent;

	FString LocalPeerId;

	UPROPERTY()
	TObjectPtr<AMultiServerBeaconHost> BeaconHost;

	UPROPERTY()
	TObjectPtr<AMultiServerBeaconHostObject> BeaconHostObject;

	UPROPERTY()
	TArray<TObjectPtr<UMultiServerPeerConnection>> PeerConnections;

	UPROPERTY(Config)
	float RetryConnectDelay;

	UPROPERTY(Config)
	float RetryConnectMaxDelay;

	TSubclassOf<AMultiServerBeaconClient> UserBeaconClass;

	// How many servers are we expecting (including ourselves). This should be at least 1.
	uint32 NumExpectedServers = 1;

	// Multi-server nodes handle ticking of their NetDrivers directly instead of letting the world tick them.
	// This allows us to control the timing of the Tick(Flush|Dispatch) and PostTick(Flush|Dispatch) functions
	// to ensure they're always called as atomic units. Since MultiServer drivers might be ticked from within
	// a NetDriver that's being ticked by the world, and the world ticks in passes (all netdrivers Tick, then
	// all netdrivers PostTick), we could end up in a situation where a MultiServer driver has Ticked, and is
	// Ticked again before the corresponding PostTick was called (if the world was allowed to tick
	// the MultiServer drivers).
	// Note UMultiServerNetDriver::SetWorld unregisters from world tick events. See UMultiServerNetDriver.
	FDelegateHandle TickDispatchDelegateHandle;
	FDelegateHandle TickFlushDelegateHandle;

	void RegisterTickEvents();
	void UnregisterTickEvents();

	void InternalTickDispatch(float DeltaSeconds);
	void InternalTickFlush(float DeltaSeconds);
};

template<class T>
T* UMultiServerNode::GetBeaconClientForRemotePeer(FStringView RemotePeerId) const
{
	return Cast<T>(GetBeaconClientForRemotePeer(RemotePeerId));
}

template<class T>
T* UMultiServerNode::GetBeaconClientForURL(const FString& InURL) const
{
	return Cast<T>(GetBeaconClientForURL(InURL));
}