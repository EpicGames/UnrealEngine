// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/PendingNetGame.h"
#include "Engine/ChildConnection.h"
#include "Engine/PackageMapClient.h"
#include "GameFramework/OnlineReplStructs.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/NetObjectFactory.h"
#include "IpNetDriver.h"
#include "IpConnection.h"
#include "UObject/SoftObjectPath.h"
#include "MultiServerProxy.generated.h"

namespace UE::MultiServerProxy::Private
{

	struct MULTISERVERREPLICATION_API FPendingBroadcastRPC
	{
		TObjectKey<const UFunction> Function;

		TArray<uint8> Parms;

		TObjectKey<const UObject> SubObject;

		FPendingBroadcastRPC() = default;
		FPendingBroadcastRPC(FPendingBroadcastRPC&& Other) = default;
		FPendingBroadcastRPC& operator=(FPendingBroadcastRPC&& Other) = default;

		FPendingBroadcastRPC(const UFunction* InFunction, const void* InParms, const UObject* InSubObject);
		FPendingBroadcastRPC(const FPendingBroadcastRPC& Other);
		FPendingBroadcastRPC& operator=(const FPendingBroadcastRPC& Other);
		~FPendingBroadcastRPC();

		void DestroyParms();
	};

} // namespace UE::MultiServerProxy::Private

enum class EProxyConnectionState : uint8
{
	Disconnected,
	Connecting,
	ConnectingPrimary,
	Connected,
	ConnectedPrimary,
	PendingReassign,
	PendingClose
};

/** 
 * A NetGUID cache used by UProxyNetDriver that doesn't assign new NetGUIDs but uses an already assigned NetGUID
 * from a backend server connection's NetGUID cache.
 */
class FProxyNetGUIDCache : public FNetGUIDCache
{
public:
	FProxyNetGUIDCache(UNetDriver* NetDriver);

	virtual FNetworkGUID AssignNewNetGUID_Server(UObject* Object);
	virtual FNetworkGUID AssignNewNetGUIDFromPath_Server(const FString& PathName, UObject* ObjOuter, UClass* ObjClass);

private:
	FNetworkGUID LookupNetGUIDFromBackendCache(UObject *Object);
};

class FProxyBackendNetGUIDCache : public FNetGUIDCache
{
public:
	FProxyBackendNetGUIDCache(UNetDriver* NetDriver);

	virtual bool IsNetGUIDAuthority() const;
};

USTRUCT()
struct FGameServerSplitJoinRequest
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<class ULocalPlayer> Player;

	UE::Net::EJoinFlags Flags;

	uint32 ClientHandshakeId;
};

/**
 * A physical connection from the proxy server to a single game server.
 *
 * All proxy client connections are multiplexed to backend servers through this physical connection.
 */ 
USTRUCT()
struct FGameServerConnectionState
{
	GENERATED_BODY()

	FGameServerConnectionState(const FURL& InGameServerURL)
	: GameServerURL(InGameServerURL)
	{
	}

	/** The URL address of the Game Server (does not indicate a live connection, only intent) */
	UPROPERTY()
	FURL GameServerURL;

	UPROPERTY()
	TObjectPtr<class UWorld> World;

	UPROPERTY()
	TObjectPtr<class UProxyBackendNetDriver> NetDriver;
	
	FName NetDriverName;

	UPROPERTY()
	TArray<TObjectPtr<class ULocalPlayer>> Players;

	UPROPERTY()
	TObjectPtr<UGameServerNotify> GameServerNotify;

	UPROPERTY()
	TArray<FGameServerSplitJoinRequest> PendingSplitJoinRequests;

	// We are only valid with a URL, but a default constructor is needed for serialization
	FGameServerConnectionState() = default;
};

/** 
 * A route from a proxy client connection to a single game server.
 */ 
USTRUCT()
struct FMultiServerProxyInternalConnectionRoute
{
	GENERATED_BODY()

	uint32 ClientHandshakeId;

	/** The client connection to the proxy (the beginning of the route). */
	UPROPERTY()
	TObjectPtr<UNetConnection> ProxyConnection;

	/** The parent connection to the backend server. */
	UPROPERTY()
	TObjectPtr<UNetConnection> ParentGameServerConnection;

	/** The actual connection to the backend server. */
	UPROPERTY()
	TObjectPtr<UNetConnection> GameServerConnection;

	/** The player used to represent this route in the game systems. */
	UPROPERTY()
	TObjectPtr<class ULocalPlayer> Player;
	
	/** Queue to hold RPCs that will be broadcast to all game servers using BroadcastToAllClientConnectedGameServers(). */
	TArray<UE::MultiServerProxy::Private::FPendingBroadcastRPC> PendingBroadcastRPCs;

	/**
	 * The player controller.
	 * 
	 * The pointer to APlayerController is const in order to make sure changes to the pointer are
	 * explicit using const_cast. This is important because accidental changes to the controller, 
	 * especially when being reassigned, can lead to difficult to find routing bugs.
	 */
	UPROPERTY()
	mutable TObjectPtr<const class APlayerController> PlayerController;

	/** The state of the route. */
	mutable EProxyConnectionState State = EProxyConnectionState::Disconnected;
	
	mutable double LastUpdateViewTargetSec;
	mutable FVector LastViewTargetPos;
};

/** 
 * State associated with the reassignment or migration of a player controller from one game server to another.
*/
USTRUCT()
struct FPlayerControllerReassignment
{
	GENERATED_BODY()

	bool bReceivedNoPawnPlayerController = false;
	bool bReceivedGamePlayerController = false;

	uint64 PreviousClientHandshakeId = 0;
	uint64 ClientHandshakeId = 0;

	UPROPERTY()
	TObjectPtr<class ULocalPlayer> PreviousPrimaryPlayer;

	UPROPERTY()
	TObjectPtr<class ULocalPlayer> PrimaryPlayer;
};

/** 
 * Intercept outgoing connection requests to game servers from the proxy server.
 *
 * Ensure that NMT_Join is sent after receiving NMT_Welcome from a game server. Normally
 * NMT_Join will be sent after a level is loaded but the proxy doesn't currently handle 
 * loading levels when connecting to a server.
 *
 * Defaults to the behavior in UPendingNetGame which normally handles all outgoing 
 * connections to a game server.
 */
UCLASS()
class UGameServerNotify : public UPendingNetGame
{
public:

	GENERATED_BODY()

	virtual void 					NotifyAcceptedConnection(UNetConnection* Connection) override;
	virtual EAcceptConnection::Type NotifyAcceptingConnection() override;
	virtual bool 					NotifyAcceptingChannel(class UChannel* Channel) override;
	virtual void 					NotifyControlMessage(UNetConnection* GameServerConnection, uint8 MessageType, class FInBunch& Bunch) override;
	virtual class ULocalPlayer* 	GetFirstGamePlayer();

	void SetProxyNetDriver(TObjectPtr<class UProxyNetDriver> InProxyNetDriver);
	void SetFlags(UE::Net::EJoinFlags InFlags) { Flags = InFlags; };

	/** Set the value to be returned by GetFirstGamePlayer(). */
	void SetFirstPlayer(TObjectPtr<class ULocalPlayer> Player);

private:

	UE::Net::EJoinFlags Flags;

	UPROPERTY()
	TObjectPtr<class ULocalPlayer> FirstPlayer;
	
	UPROPERTY()
	TObjectPtr<class UProxyNetDriver> ProxyNetDriver;
};

/** 
 * Intecept incoming from clients to the proxy server.
 *
 * Intercept NMT_Join when a client connects to the proxy, establishes a connection to
 * a game server and performs the logic required to associate these two connections and
 * forward state replicated from the game server to the client.
 * 
 * Defaults to the behavior in UWorld which normally handles all incoming game server 
 * connections.
 */
UCLASS()
class UProxyListenerNotify : public UObject, 
							 public FNetworkNotify
{
public:

	GENERATED_BODY()

	void SetProxyNetDriver(TObjectPtr<class UProxyNetDriver> InProxyNetDriver);

	virtual void 					NotifyAcceptedConnection(UNetConnection* Connection) override;
	virtual EAcceptConnection::Type NotifyAcceptingConnection() override;
	virtual bool 					NotifyAcceptingChannel(class UChannel* Channel) override;
	virtual void 					NotifyControlMessage(UNetConnection* ProxyConnection, uint8 MessageType, class FInBunch& Bunch) override;

private:

	/** Start connecting an incoming proxy connection to a game server. */
	void ConnectToGameServer(UNetConnection* ProxyConnection, 
							 int32 GameServerConnectionStateIndex, 
							 FGameServerConnectionState* GameServerConnectionState, 
							 UE::Net::EJoinFlags Flags);

	UPROPERTY()
	TObjectPtr<class UProxyNetDriver> ProxyNetDriver;
};

/** 
* A network connection used by UProxyBackendNetDriver.
 */
UCLASS()
class MULTISERVERREPLICATION_API UProxyBackendNetConnection : public UIpConnection
{
	GENERATED_BODY()

public:

	virtual void HandleClientPlayer(APlayerController* PC, UNetConnection* NetConnection);
};

/**
 * A child network connection used by UProxyBackendNetDriver.
 */
UCLASS()
class MULTISERVERREPLICATION_API UProxyBackendChildNetConnection : public UChildConnection
{
	GENERATED_BODY()

public:

	virtual void HandleClientPlayer(APlayerController* PC, UNetConnection* NetConnection);
	
protected:

	virtual void RemoveLocalPlayer(UGameInstance* GameInstance, ULocalPlayer* LocalPlayer) override;
};

/** 
 * A driver that is used by UProxyNetDriver to connect to backend game servers.
 */
UCLASS()
class MULTISERVERREPLICATION_API UProxyBackendNetDriver : public UIpNetDriver
{
	GENERATED_BODY()

public:

	void SetProxyNetDriver(TObjectPtr<UProxyNetDriver> InParentNetDriver);
	TObjectPtr<UProxyNetDriver> GetProxyNetDriver();

	virtual bool InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error);
	virtual void Shutdown() override;
	virtual void ForwardRemoteFunction(UObject* RootObject, UObject* SubObject, UFunction* Function, void* Parms) override;
	virtual bool ShouldSkipRepNotifies() const override;
	virtual bool ShouldClientDestroyActor(AActor* Actor, EChannelCloseReason CloseReason) const;

	virtual bool ShouldUpdateStats() const override;
	virtual bool ShouldRegisterMetricsDatabaseListeners() const override;

private:

	void OnIrisRootObjectPostInit(UE::Net::FNetRefHandle RootHandle, UObject* RootObject, UE::Net::FNetObjectFactoryId FactoryId);
	void OnIrisRootObjectDetached(UE::Net::FNetRefHandle RootHandle, UObject* RootObject, UE::Net::EDetachReason Reason);
	void OnIrisSubObjectPostInit(UE::Net::FNetRefHandle SubObjectHandle, UObject* SubObject, UE::Net::FNetRefHandle RootObjectHandle, UObject* RootObject, UE::Net::FNetObjectFactoryId FactoryId);
	void OnIrisSubObjectDetached(UE::Net::FNetRefHandle SubObjectHandle, UObject* SubObject, UE::Net::FNetRefHandle RootObjectHandle, UObject* RootObject, UE::Net::EDetachReason Reason);

	UPROPERTY()
	TObjectPtr<UProxyNetDriver> ProxyNetDriver;

	FDelegateHandle OnIrisRootObjectPostInitHandle;
	FDelegateHandle OnIrisRootObjectDetachedHandle;
	FDelegateHandle OnIrisSubObjectPostInitHandle;
	FDelegateHandle OnIrisSubObjectDetachedHandle;
};

/**
* A network proxy that intercepts and forwards UE game network connections to backend game servers.
*
* The proxy externally behaves the same as a normal game server when game clients connect and
* as a normal client when connecting to game servers. This means that there is no need for the
* clients and game servers that the proxy is connected to have any special proxy-aware configuration.
*
* Internally, the proxy is made up of an instance of UProxyNetDriver that listens for incoming 
* connections, known as proxy connections, and an instance of UProxyBackendNetDriver for each 
* connection to a backend game server. State from the backend servers is replicated into a single,
* shared UWorld and the listening UProxyNetDriver replicates that state out to proxy connections.
*
* All actors replicated to the proxy from remote game servers will have the same role as a client
* (ROLE_SimulatedProxy or ROLE_AutonomousProxy) and will be replicated as-is to the proxy client.
*
* When a proxy connection (UNetConnection) is opened in UProxyNetDriver it opens a game server 
* connection (UNetConnection) to each registered backend server.
*
* For each proxy connection one of the game servers are considered the primary game server. This
* is the game server that spawns the proxy client's pawn, player controller, receive RPCs from
* the proxy connection, and send RPCs to the proxy connection. The other game servers are considered
* non-primary game servers and only replicate state relevant to that connection to the proxy.
*
* When connecting to non-primary game servers the game server will spawn a ANoPawnPlayerController 
* player controller, and not spawn a pawn. These connections will replicate state from the game 
* server but not maintain a player presence.
*/
UCLASS()
class MULTISERVERREPLICATION_API UProxyNetDriver : public UIpNetDriver
{
	GENERATED_BODY()

public:
	
	/** Register a game server. */
	void RegisterGameServer(const FURL& GameServerURL);

	/** Cycle through game servers as primary game servers each time a client connects to the proxy. */
	void SetCyclePrimaryGameServer(bool bEnable);

	/** Return true if all registered servers are connected. */
	bool IsConnectedToAllGameServers() const;

	/** Enumerate through all outgoing connections to game servers. */
	int32 GetGameServerConnectionCount() const;
	FGameServerConnectionState* GetGameServerConnection(int32 Index);

	/** Return the number of child connections to the game servers. */
	int32 GetGameServerChildConnectionCount() const;

	/** UNetDriver interface function. */
	virtual bool InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error) override;
	virtual void SetupNetworkMetrics() override;
	virtual bool InitConnect(class FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error) override;
	virtual void ForwardRemoteFunction(UObject* RootObject, UObject* SubObject, UFunction* Function, void* Parms) override;
	virtual bool ShouldReplicateFunction(AActor* Actor, UFunction* Function) const override;
	virtual void AddNetworkActor(AActor* Actor) override;
	virtual void RemoveNetworkActor(AActor* Actor) override;
	virtual bool ShouldCallRemoteFunction(UObject* Object, UFunction* Function, const FReplicationFlags& RepFlags) const;
	virtual int32 ServerReplicateActors(float DeltaSeconds) override;
	virtual bool CanDowngradeActorRole(UNetConnection* ProxyConnection, AActor* Actor) const override;
	virtual void Shutdown() override;
	virtual void FinishDestroy() override;
	virtual void TickFlush(float DeltaSeconds) override;
	
	/** Get the next identifier for outgoing connections to game servers. */
	int32 GetNextGameServerClientId();

	/** Get the next connection handshake id. */
	uint32 GetNextClientHandshakeId();
	
	/** Called when the player controller associated with a connection is changed (either at the end of initial connection handshake, or if changed after successfully connected). */
	void GameServerAssignPlayerController(UNetConnection* ChildGameServerConnection, UNetConnection* ParentGameServerConnection, APlayerController* PlayerController);

	/** Return a pointer to the shared NetGUID cache used by backend network connections. */
	TSharedPtr<FNetGUIDCache> GetSharedBackendNetGuidCache();

	/** Forward an RPC to all primary and non-primary game servers for a proxy client connection using PlayerToProxyConnection. */
	void BroadcastToAllClientConnectedGameServers(const ULocalPlayer* Player, const UFunction* Function, const void* Parms, const UObject* SubObject);

	/** Return the proxy connection for a local player, or nullptr if not found. */
	UNetConnection* GetProxyConnectionForPlayer(const ULocalPlayer* Player) const;

	/** Called when an incoming proxy connection has been closed and closes connections to backend servers. */
	void HandleClosedProxyConnection(UNetConnection* ProxyConnection);
	
	/** Send any split join requests that have been queued up to the game server. */
	void FlushSplitJoinRequests(FGameServerConnectionState* GameServerConnectionState);

	/** Flush any broadcast RPCs that were queued while a route was not yet connected. */
	void FlushPendingBroadcastRPCs(FMultiServerProxyInternalConnectionRoute* Route);

private:

	friend class UProxyListenerNotify;

	// Classes in EngineTest that test the behavior of UProxyNetDriver.
	friend class FNetTestProxyConnectionRouting;
	friend class FNetTestProxyCloseSecondClientBeforeConnectedToBackend;
	friend class FNetTestProxyCloseFirstClientBeforeBackendParentConnected;
	friend class FNetTestProxyAllowListExecOnProxy;
	friend class FNetTestProxyNonAllowListedSkipsProxy;
	friend class FNetTestProxyNonBroadcastPrimaryOnly;
	friend class FNetTestProxyBroadcastReachesAllServers;
	friend class FNetTestProxyGCCleansPendingBroadcastRPCs;

	friend class UProxyBackendNetDriver;

	/** Close a route's connection and perform any clean up. */
	void CloseAndCleanUpConnectionRoute(const FMultiServerProxyInternalConnectionRoute* Route);

	/** Called when the connection handshake to a game server is completed (i.e. the first player controller has been replicated on that connection). */
	void FinalizeGameServerConnection(FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController);

	/** Called when a player controller is switching on the remote game server (i.e. during migration). */
	void ReassignPlayerController(const FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController);

	void ProcessConnectionState_Connecting(FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController);
	void ProcessConnectionState_ConnectingPrimary(FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController);
	void ProcessConnectionState_Connected(const FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController);
	void ProcessConnectionState_ConnectedPrimary(const FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController);
	void ProcessConnectionState_Reassign(const FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController);
	void ProcessConnectionState_PendingClose(FMultiServerProxyInternalConnectionRoute* Route, UNetConnection* GameServerConnection, APlayerController* PlayerController);

	void UpdateProxyNetworkMetrics();
	void AggregateBackendNetDriverMetrics();
	void ResetProxyNetworkMetrics();

	/** Clean up any internal data structures that hold weak object references that are now invalid. */
	void PreGarbageCollect();
	void PostGarbageCollect();

	/** 
	 * Send an RPC correct proxy connection based on the actor's owner and local player.
	 *
	 * This function won't perform logic to send the RPC; that logic is expected to be called as part of the function
	 * argument RPCSendingFunction. RPCSendingFunction will be provided the derived proxy connection, if one can be found, as an argument.
	 */
	void RedirectRPCToProxyConnection(AActor* Actor, UFunction* Function, TFunction<void(UNetConnection*)> RPCSendingFunction);

	/** (Iris only) Configure the connection to own this particular actor (required for replication conditionals and internal filtering logic). */
	void SetProxyConnectionAsOwner(UNetConnection* ProxyConnection, const AActor* Actor);

	/** 
	 * Receive a new game player controller for an already open route.
	 * 
	 * This route becomes a route to a primary game server.
	 */
	void ReceivedReassignedGamePlayerController(const FMultiServerProxyInternalConnectionRoute* Route);
	
	/** 
	 * Receive a new no-pawn player controller for an already open route.
	 * 
	 * This route stops being a route to a primary game server.
	 */
	void ReceivedReassignedNoPawnPlayerController(const FMultiServerProxyInternalConnectionRoute* Route);

	/** Perform any logic that depends on the order of game and no pawn player controllers being re-assigned. */
	void FinalizePlayerControllerReassignment(UNetConnection* ProxyConnection, const FPlayerControllerReassignment& Migration);

	/** Set all of the configuration options that disable executing actor functionality or game specific code. */
	void DisableActorLogicAndGameCode();

	/** Prepare state on the proxy to be used for relevancy when replicating to proxy connections. */
	void PrepareStateForRelevancy();

	/** Update the view target on player controllers used by proxy client connections. */
	void UpdateLocalViewTarget(const APlayerController* PrimaryPlayerController);

	/** Set the view target on no pawn connections to the game server. */
	void SetRemoteViewTarget(const class ANoPawnPlayerController* PlayerController, FVector ViewTargetPos);

	/** Return a proxy connection from a primary player (a player on a primary game server). */
	UNetConnection* GetProxyConnectionFromPrimaryPlayer(ULocalPlayer* Player);
	
	/** Returns the internal route that uses a player controller. */
	const FMultiServerProxyInternalConnectionRoute* GetRouteWithPlayerController(const APlayerController* PlayerController);

	/**
	 * Map client handshake ids to a route data structure.
	 * 	
	 * The size of this map is (m*n), where m is the number of proxy connections and n is the number of 
	 * game servers. For this reason enumerating of this map should be avoided and rather use the functions
	 * GetRouteWithPlayerController(), GetRouteWithClientHandshakeId(), and other internal map data structures.
	 */
	UPROPERTY()
	TMap<uint32, FMultiServerProxyInternalConnectionRoute> InternalRoutes;

	/** Map a player to it's primary game server player controller. */
	UPROPERTY()
	TMap<TObjectPtr<class ULocalPlayer>, TObjectPtr<const APlayerController>> PlayerToPrimaryGameServerPlayerController;

	/** Map a local player to it's proxy net connection. Multiple players will point to the same proxy net connection. */
	TMap<TObjectKey<class ULocalPlayer>, TObjectPtr<UNetConnection>> PlayerToProxyConnection;

	FDelegateHandle ProxyPostGarbageCollectHandle;
	FDelegateHandle ProxyPreGarbageCollectHandle;

	UPROPERTY()
	TMap<TObjectPtr<UNetConnection>, FPlayerControllerReassignment> DeferredReassignment;

	/** Map a proxy connection to the primary game server player controller. */
	UPROPERTY()
	TMap<TObjectPtr<UNetConnection>, TObjectPtr<const APlayerController>> ProxyConnectionToPrimaryGameServerPlayerController;

	/** Net drivers and associated state used to connect to backend game servers. */
	UPROPERTY()
	TArray<FGameServerConnectionState> GameServerConnections;

	/** Proxy listener handshake logic. */
	UPROPERTY()
	TObjectPtr<UProxyListenerNotify> ProxyListenerNotify;
	
	/** A single NetGUID cache used by all netdrivers used to connect to backend servers. */
	TSharedPtr<FNetGUIDCache> SharedBackendNetGuidCache;

	int32 NextGameServerClientId = 0;
	
	uint32 NextClientHandshakeId = 123;
	
	// The primary game server to use for the next primary game client.
	int32 PrimaryGameServerForNextClient = 0;

	// After a client connects to the proxy increment the value of PrimaryGameServerForNextClient.
	bool bCyclePrimaryGameServer = false;

	// As a client connects to the proxy, randomize PrimaryGameServerForNextClient to one of the known game servers.
	bool bRandomizePrimaryGameServerForNextClient = false;

	/** RPCs from the client that must be forwarded to all game servers (not just the primary). */
	TSet<FSoftObjectPath> BroadcastToAllGameServersFunctions;

	/** RPCs that are allow-listed to be executed on the proxy when forwarding (both client->server and server->client). */
	TSet<FSoftObjectPath> AllowListedProxyFunctions;
};

/** 
* A network connection used by UProxyNetDriver.
*/
UCLASS()
class MULTISERVERREPLICATION_API UProxyNetConnection : public UIpConnection
{
	GENERATED_BODY()

public:

	virtual void CleanUp() override;
};
