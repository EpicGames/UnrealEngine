// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/** See MultiServerNode for more details about multi-server networking as a whole. */
class MULTISERVERREPLICATION_API FMultiServerReplicationModule : public IModuleInterface
{
public:

	FMultiServerReplicationModule() {}
	virtual ~FMultiServerReplicationModule() {}

	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override {}
	virtual bool SupportsDynamicReloading() override { return false; }
	virtual bool SupportsAutomaticShutdown() override { return false; }

	/** 
	 * Return true if the process is running as a proxy.
	 *
	 * This function should *NOT* be used in code that is intended to run from within PIE.
	 */
	static bool IsRunningAsProxy();

private:

	/**
 	 * Configures a game server to act as a proxy.
	 *
	 * A game server is configured as a proxy when the cvar `net.proxy.Enabled` is set to `true` at start up.
	 *
	 * These are the methods in which code on a game server can determine if it's running as a proxy:
	 *		* UReplicationSystem::GetProxyType() or UNetDriver::GetProxyType()
	 *		* FMultiServerReplicationModule::IsRunningAsProxy()
	 *		* Read the value of the cvar `net.proxy.Enabled`.
	 *
	 * The preferred method is to use `GetProxyType()` which is world-scoped, and only use `IsRunningAsProxy()`
	 * or reading the cvar directly when `GetProxyType()` is unavailable (e.g. before the driver has been initialized).
	 */
	void SetupProxy();
};


