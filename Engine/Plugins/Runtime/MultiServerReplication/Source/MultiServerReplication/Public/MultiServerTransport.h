// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Containers/SpscQueue.h"
#include "UObject/GCObject.h"
#include "Templates/SubclassOf.h"
#include "MultiServerTransport.generated.h"

class FMultiServerTransport;
class UMultiServerTransportEndpoint;

// This is the base class for MultiServerTransport endpoints. You are meant to
// derive from this and put your RPCs in the derived class
UCLASS()
class MULTISERVERREPLICATION_API UMultiServerTransportEndpoint : public UObject
{
	friend class FMultiServerTransport;

	// this indexes into the internal FMultiServerTransport PeerStates array
	int32 PeerIndex = -1;
	FMultiServerTransport* Transport = nullptr;

public:
	GENERATED_BODY()

	FString GetLocalPeerId() const;
	FString GetRemotePeerId() const;

	UWorld* GetWorld() const;

	// UObject overrides to redirect RPCs
	virtual int32 GetFunctionCallspace( UFunction* Function, FFrame* Stack ) override;
	virtual bool CallRemoteFunction( UFunction* Function, void* Parms, struct FOutParmRec* OutParms, FFrame* Stack ) override;
};

class MULTISERVERREPLICATION_API FMultiServerTransport : public FRunnable, public FGCObject
{
	friend class UMultiServerTransportEndpoint;

public:
	FMultiServerTransport(
		const FString& SharedSessionKeySecretString,
		const FString& LocalPeerId,
		uint16 ListenPort,
		const FString& LocalServerAddress,
		const TArray<FString>& AllServerAddresses,
		TSubclassOf<UMultiServerTransportEndpoint> EndpointClass);

	~FMultiServerTransport();

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	// FRunnable interface
	virtual uint32 Run() override;

	virtual void Stop() override
	{
		bIsRunning = false;
		SignalThreadWake();
	}

	int32 GetNumExpectedConnections() const
	{
		int32 TotalAddresses = AllServerAddresses.Num();
		return (TotalAddresses > 0) ? TotalAddresses-1 : 0;
	}

	void SetWorld(UWorld* InWorld)
	{
		World = InWorld;
	}

	UWorld* GetWorld() const
	{
		return World;
	}

	FString GetLocalPeerId() const
	{
		return LocalPeerId;
	}

	// Processes incoming RPCs on all peer endpoints.
	// Returns true if the transport is still operational.
	// Returns false if a peer disconnected (transport is now dormant).
	bool GameThreadTick();

	UMultiServerTransportEndpoint* GetEndpointForRemotePeer(FStringView RemotePeerId) const;

	template<class T>
	T* GetEndpointForRemotePeer(FStringView RemotePeerId) const;

	void ForEachEndpoint(TFunctionRef<void(UMultiServerTransportEndpoint*)> Operation) const;

private:
	static constexpr uint32 MagicNumber = 0x5cbdc807u;
	static constexpr uint32 NonceSize = 12;
	static constexpr uint32 SharedSessionKeySize = 32;
	static constexpr uint32 HMACHashSize = 20; // SHA-1 output
	static constexpr uint32 AuthMessageSize = NonceSize + HMACHashSize;
	static constexpr double ConnectTimeout = 30.0; // 30-second timeout

	struct FPendingPeer
	{
		double CreationTimeSeconds = 0.0;
		FUniqueSocket Socket;

		uint32 MagicNumberBuffer = 0;
		uint32 MagicNumberOffset = 0;

		uint8 AuthMessageBuffer[AuthMessageSize];
		uint32 AuthMessageBufferOffset = 0;

		enum class EPendingPeerTickResult
		{
			StillPending,
			IsInvalid,
			TimedOut,
			PromoteToPeer,
		};

		EPendingPeerTickResult TickPendingPeer(const uint8* SharedSessionKeySecret);
		void HandlePendingPeerRecv();
	};

	struct FPeer
	{
		TObjectPtr<UMultiServerTransportEndpoint> Endpoint;
		FUniqueSocket Socket;

		FString RemotePeerId;

		// this is a fixed size circular buffer of bytes ready to
		// be send out of the socket.
		static const uint32 SendStagingBufferSize = 2*1024*1024;
		TUniquePtr<uint8[]> SendStagingBuffer;
		uint32 SendStagingBufferPutIndex = 0;
		uint32 SendStagingBufferGetIndex = 0;
			
		// we can be in a state where we haven't fully copied the
		// next message to go out from the message queue into the
		// staging buffer. We try to stage as much as possible,
		// and this offset lets us keep track of how far into the
		// message we have staged so far.
		int32 SendStagingBufferPendingMessageOffset = -1;

		// send queue
		TSpscQueue<TArray<uint8>> SendMessageQueue;

		// recv queue
		TSpscQueue<TArray<uint8>> RecvMessageQueue;

		// Fixed-size receive buffer. The socket reads directly into
		// this buffer, and then we assemble messages out of it. This
		// allows assembling more than one message per socket pump
		// when messages are small.
		static const uint32 RecvStagingBufferSize = 2*1024*1024;
		TUniquePtr<uint8[]> RecvStagingBuffer;
		uint32 RecvStagingBufferPutIndex = 0;
		uint32 RecvStagingBufferGetIndex = 0;

		// Message being assembled from the receive buffer.
		// When Num()==0, we are waiting for a 4-byte size header.
		// When Num()>0, we are filling the body, and
		// RecvMessageBufferOffset tracks how far along we are.
		TArray<uint8> RecvMessageBuffer;
		uint32 RecvMessageBufferOffset = 0;

		enum class EPeerRecvResult
		{
			Ok,
			Disconnected,
			HandshakeReceived,
		};

		void StageOutgoingMessages();
		bool HandlePeerSend();
		EPeerRecvResult HandlePeerRecv();
	};

	void InitiateOutboundConnections();
	void CreatePendingPeerState(FUniqueSocket&& PeerSocket);
	void CreatePeerState(FUniqueSocket&& PeerSocket, const uint8* AuthMessageBytes, uint32 AuthMessageBytesSize);
	void SendToPeer(int32 PeerIndex, TArray<uint8>&& Message);
	void SignalThreadWake();
	
	uint8 SharedSessionKeySecret[SharedSessionKeySize];
	UWorld* World = nullptr;

	TArray<TUniquePtr<FPendingPeer>> PendingPeerStates;
	TArray<TUniquePtr<FPeer>> PeerStates;
	int32 ConnectedEndpointCount = 0;

	TAtomic<bool> bIsRunning;
	TUniquePtr<FRunnableThread> TransportThread;
	TSpscQueue<bool> AllEndpointsConnectedEventQueue;

	FUniqueSocket ListenSocket;

	// this is a loopback socket used to signal the transport thread to wake up
	// when the game thread pushes a message to send into the queue
	FUniqueSocket LoopbackSendSocket; // this is the endpoint that the game thread writes to
	FUniqueSocket LoopbackRecvSocket; // this is the endpoint that the net thread reads from

	TArray<FString> AllServerAddresses;
	int32 LocalPeerIndex = 0; // our index into AllServerAddresses
	FString LocalPeerId;
};

template<class T>
T* FMultiServerTransport::GetEndpointForRemotePeer(FStringView RemotePeerId) const
{
	return Cast<T>(GetEndpointForRemotePeer(RemotePeerId));
}

inline UWorld* UMultiServerTransportEndpoint::GetWorld() const
{
	return Transport->GetWorld();
}
